// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.IO.Compression;
using System.Runtime.InteropServices;
using System.Runtime.Remoting;
using System.Runtime.Remoting.Channels;
using System.Runtime.Remoting.Channels.Tcp;
using System.Runtime.Remoting.Messaging;
using System.Threading;

using AgentInterface;
using System.Reflection;

namespace NSwarm
{
	internal static class DebugLog
	{
		[Conditional("DEBUG")]
		internal static void Write (string msg)
		{
			Console.WriteLine(msg);
		}
	}

	/*
	 * PerfTiming
	 *
	 * An instance of a timing object. It tracks a single timing; the total time and the number of calls
	 */
	public class PerfTiming
	{
		public String			Name;
		public Stopwatch		StopWatchTimer;
		public Int32			Count;
		public bool				Accumulate;
		public Int64			Counter;

		public PerfTiming(String InName, bool InAccumulate)
		{
			Name = InName;
			StopWatchTimer = new Stopwatch();
			Count = 0;
			Accumulate = InAccumulate;
			Counter = 0;
		}

		public void Start()
		{
			StopWatchTimer.Start();
		}

		public void Stop()					
		{
			Count++;
			StopWatchTimer.Stop();
		}

		public void IncrementCounter(Int64 Adder)
		{
			Counter += Adder;
		}
	}

	/*
	 * PerfTimer
	 *
	 * Tracks a dictionary of PerfTimings
	 */
	public class PerfTimer
	{
		public Stack<PerfTiming>						LastTimers;
		ReaderWriterDictionary<String, PerfTiming>		Timings;

		public PerfTimer()
		{
			Timings = new ReaderWriterDictionary<String, PerfTiming>();
			LastTimers = new Stack<PerfTiming>();
		}

		public void Start(String Name, bool Accum, Int64 Adder)
		{
			Monitor.Enter(LastTimers);

			PerfTiming Timing = null;
			if (!Timings.TryGetValue(Name, out Timing))
			{
				Timing = new PerfTiming(Name, Accum);
				Timings.Add(Name, Timing);
			}

			LastTimers.Push(Timing);

			Timing.IncrementCounter(Adder);
			Timing.Start();

			Monitor.Exit(LastTimers);
		}

		public void Stop()
		{
			Monitor.Enter(LastTimers);

			PerfTiming Timing = LastTimers.Pop();
			Timing.Stop();

			Monitor.Exit(LastTimers);
		}

		public String DumpTimings()
		{
			String Output = "";

			double TotalTime = 0.0;
			foreach (PerfTiming Timing in Timings.Values)
			{
				if (Timing.Count > 0)
				{
					double Elapsed = Timing.StopWatchTimer.Elapsed.TotalSeconds;
					double Average = (Elapsed * 1000000.0) / Timing.Count;
					Output += Timing.Name.PadLeft(30) + " : " + Elapsed.ToString("F3") + "s in " + Timing.Count + " calls (" + Average.ToString("F0") + "us per call)";

					if (Timing.Counter > 0)
					{
						Output += " (" + (Timing.Counter / 1024) + "k)";
					}

					Output += "\n";
					if (Timing.Accumulate)
					{
						TotalTime += Elapsed;
					}
				}
			}

			Output += "\nTotal time inside Swarm: " + TotalTime.ToString("F3") + "s\n";

			return Output;
		}
	}

	///////////////////////////////////////////////////////////////////////////////

	/**
	 * Connection configuration parameters, filled in by OpenConnection
	 */
	public class AgentConfiguration
	{
		public AgentConfiguration()
		{
			AgentProcessID = -1;
			AgentCachePath = null;
			AgentJobGuid = null;
			IsPureLocalConnection = false;
		}
		
		// Process ID of the owning process for the agent, which can be used to
		// monitor it for crashes or hangs
		public Int32			AgentProcessID;
		
		// The full path of the cache directory this agent is using
		public String			AgentCachePath;
		
		// The GUID of the job the agent has associated this connection with, if any
		public AgentGuid		AgentJobGuid;

		// An indication of whether we're considered a "pure" local connection by the
		// Agent, potentially relieving us from Agent coordination when using Channels
		public bool				IsPureLocalConnection;
	}
	
	///////////////////////////////////////////////////////////////////////////////

	/**
	 * A wrapper to abstract away the complexity of asynchronous calls for the
	 * Agent API and the monitoring of the connection
	 */
	public class IAgentInterfaceWrapper
	{
		public IAgentInterfaceWrapper()
		{
			// TODO: Make URL configurable
			Connection = (IAgentInterface)Activator.GetObject(typeof(IAgentInterface), "tcp://127.0.0.1:8008/SwarmAgent");
			ConnectionDroppedEvent = new ManualResetEvent(false);
		}
		
		public void SignalConnectionDropped()
		{
			ConnectionDroppedEvent.Set();
		}
		
		///////////////////////////////////////////////////////////////////////////
		
		// A duplication of the IAgentInterface API which this class wraps
		public Int32 OpenConnection(Process AgentProcess, bool AgentProcessOwner, Int32 LocalProcessID, ELogFlags LoggingFlags, out AgentConfiguration NewConfiguration)
		{
			OpenConnectionDelegate DOpenConnection = Connection.OpenConnection;
			
			// Set up the versioned hashtable input parameters
			Hashtable InParameters = new Hashtable();
			InParameters["Version"] = ESwarmVersionValue.VER_1_0;
			InParameters["ProcessID"] = LocalProcessID;
			InParameters["ProcessIsOwner"] = (AgentProcessOwner == true ? true : false);
			InParameters["LoggingFlags"] = LoggingFlags;
			
			Hashtable OutParameters = null;
			IAsyncResult Result = DOpenConnection.BeginInvoke(InParameters, ref OutParameters, null, null);
			
			// This will wait with an occasional wake up to check to see if the
			// agent process is still alive and kicking (avoids infinite wait,
			// allows for very long start up times while debugging)
			Int32 StartupSleep = 1000;
			while ((Result.AsyncWaitHandle.WaitOne(StartupSleep) == false) &&
			      (AgentProcess.HasExited == false) &&
			      (AgentProcess.Responding == true))
			{
				// While the application is alive and responding, wait
				DebugLog.Write("[OpenConnection] Waiting for agent to respond ...");
			}

			if (Result.IsCompleted)
			{
				// If the invocation didn't fail, end to get the result
				Int32 ReturnValue = DOpenConnection.EndInvoke(ref OutParameters, Result);
				if (OutParameters != null)
				{
					if ((ESwarmVersionValue)OutParameters["Version"] == ESwarmVersionValue.VER_1_0)
					{
						NewConfiguration = new AgentConfiguration();
						NewConfiguration.AgentProcessID = (Int32 )OutParameters["AgentProcessID"];
						NewConfiguration.AgentCachePath = (String )OutParameters["AgentCachePath"];
						NewConfiguration.AgentJobGuid = (AgentGuid )OutParameters["AgentJobGuid"];
						
						if (OutParameters.ContainsKey("IsPureLocalConnection"))
						{
							NewConfiguration.IsPureLocalConnection = (bool)OutParameters["IsPureLocalConnection"];
						}
						
						// Complete and successful
						return ReturnValue;
					}
				}
			}
			// Otherwise, error
			NewConfiguration = null;
			return Constants.ERROR_CONNECTION_DISCONNECTED;
		}
		
		///////////////////////////////////////////////////////////////////////////
		
		public Int32 CloseConnection(Int32 ConnectionHandle)
		{
			CloseConnectionDelegate DCloseConnection = Connection.CloseConnection;
			
			// Invoke the method, then wait for it to finish or to be notified that the connection dropped
			Hashtable InParameters = null;
			Hashtable OutParameters = null;
			IAsyncResult Result = DCloseConnection.BeginInvoke(ConnectionHandle, InParameters, ref OutParameters, null, null);
			WaitHandle.WaitAny(new WaitHandle[]{ Result.AsyncWaitHandle, ConnectionDroppedEvent });
			
			// If the method completed normally, return the result
			if (Result.IsCompleted)
			{
				// If the invocation completed, success
				return DCloseConnection.EndInvoke(ref OutParameters, Result);
			}
			// Otherwise, error
			return Constants.ERROR_CONNECTION_DISCONNECTED;
		}
		
		///////////////////////////////////////////////////////////////////////////
		
		public Int32 SendMessage(Int32 ConnectionHandle, AgentMessage NewMessage)
		{
			SendMessageDelegate DSendMessage = Connection.SendMessage;
			
			// Set up the versioned hashtable input parameters
			Hashtable InParameters = new Hashtable();
			InParameters["Version"] = ESwarmVersionValue.VER_1_0;
			InParameters["Message"] = NewMessage;
			
			// Invoke the method, then wait for it to finish or to be notified that the connection dropped
			Hashtable OutParameters = null;
			IAsyncResult Result = DSendMessage.BeginInvoke(ConnectionHandle, InParameters, ref OutParameters, null, null);
			WaitHandle.WaitAny(new WaitHandle[]{ Result.AsyncWaitHandle, ConnectionDroppedEvent });
			
			// If the method completed normally, return the result
			if (Result.IsCompleted)
			{
				// If the invocation completed, success
				return DSendMessage.EndInvoke(ref OutParameters, Result);
			}
			// Otherwise, error
			return Constants.ERROR_CONNECTION_DISCONNECTED;
		}
		
		///////////////////////////////////////////////////////////////////////////
		
		public Int32 GetMessage(Int32 ConnectionHandle, out AgentMessage NextMessage, Int32 Timeout)
		{
			GetMessageDelegate DGetMessage = Connection.GetMessage;
			
			// Set up the versioned hashtable input parameters
			Hashtable InParameters = new Hashtable();
			InParameters["Version"] = ESwarmVersionValue.VER_1_0;
			InParameters["Timeout"] = (Int32)Timeout;
			
			// Invoke the method, then wait for it to finish or to be notified that the connection dropped
			Hashtable OutParameters = null;
			IAsyncResult Result = DGetMessage.BeginInvoke(ConnectionHandle, InParameters, ref OutParameters, null, null);
			WaitHandle.WaitAny(new WaitHandle[]{ Result.AsyncWaitHandle, ConnectionDroppedEvent });
			
			// If the method completed normally, return the result
			if (Result.IsCompleted)
			{
				// If the invocation didn't fail, end to get the result
				Int32 ReturnValue = DGetMessage.EndInvoke(ref OutParameters, Result);
				if (OutParameters != null)
				{
					if((ESwarmVersionValue)OutParameters["Version"] == ESwarmVersionValue.VER_1_0)
					{
						NextMessage = (AgentMessage )OutParameters["Message"];
						
						// Complete and successful
						return ReturnValue;
					}
				}
			}
			// Otherwise, error
			NextMessage = null;
			return Constants.ERROR_CONNECTION_DISCONNECTED;
		}
		
		///////////////////////////////////////////////////////////////////////////
		
		public Int32 AddChannel(Int32 ConnectionHandle, String FullPath, String ChannelName)
		{
			AddChannelDelegate DAddChannel = Connection.AddChannel;
			
			// Set up the versioned hashtable input parameters
			Hashtable InParameters = new Hashtable();
			InParameters["Version"] = ESwarmVersionValue.VER_1_0;
			InParameters["FullPath"] = FullPath;
			InParameters["ChannelName"] = ChannelName;
			
			// Invoke the method, then wait for it to finish or to be notified that the connection dropped
			Hashtable OutParameters = null;
			IAsyncResult Result = DAddChannel.BeginInvoke(ConnectionHandle, InParameters, ref OutParameters, null, null);
			WaitHandle.WaitAny(new WaitHandle[]{ Result.AsyncWaitHandle, ConnectionDroppedEvent });
			
			// If the method completed normally, return the result
			if (Result.IsCompleted)
			{
				// If the invocation completed, success
				return DAddChannel.EndInvoke(ref OutParameters, Result);
			}
			// Otherwise, error
			return Constants.ERROR_CONNECTION_DISCONNECTED;
		}
		
		///////////////////////////////////////////////////////////////////////////
		
		public Int32 TestChannel(Int32 ConnectionHandle, String ChannelName)
		{
			TestChannelDelegate DTestChannel = Connection.TestChannel;
			
			// Set up the versioned hashtable input parameters
			Hashtable InParameters = new Hashtable();
			InParameters["Version"] = ESwarmVersionValue.VER_1_0;
			InParameters["ChannelName"] = ChannelName;
			
			// Invoke the method, then wait for it to finish or to be notified that the connection dropped
			Hashtable OutParameters = null;
			IAsyncResult Result = DTestChannel.BeginInvoke(ConnectionHandle, InParameters, ref OutParameters, null, null);
			WaitHandle.WaitAny(new WaitHandle[]{ Result.AsyncWaitHandle, ConnectionDroppedEvent });
			
			// If the method completed normally, return the result
			if (Result.IsCompleted)
			{
				// If the invocation completed, success
				return DTestChannel.EndInvoke(ref OutParameters, Result);
			}
			// Otherwise, error
			return Constants.ERROR_CONNECTION_DISCONNECTED;
		}
		
		///////////////////////////////////////////////////////////////////////////
		
		public Int32 OpenChannel(Int32 ConnectionHandle, String ChannelName, EChannelFlags ChannelFlags)
		{
			OpenChannelDelegate DOpenChannel = Connection.OpenChannel;
			
			// Set up the versioned hashtable input parameters
			Hashtable InParameters = new Hashtable();
			InParameters["Version"] = ESwarmVersionValue.VER_1_0;
			InParameters["ChannelName"] = ChannelName;
			InParameters["ChannelFlags"] = ChannelFlags;
			
			// Invoke the method, then wait for it to finish or to be notified that the connection dropped
			Hashtable OutParameters = null;
			IAsyncResult Result = DOpenChannel.BeginInvoke(ConnectionHandle, InParameters, ref OutParameters, null, null);
			WaitHandle.WaitAny(new WaitHandle[]{ Result.AsyncWaitHandle, ConnectionDroppedEvent });
			
			// If the method completed normally, return the result
			if (Result.IsCompleted)
			{
				// If the invocation completed, success
				return DOpenChannel.EndInvoke(ref OutParameters, Result);
			}
			// Otherwise, error
			return Constants.ERROR_CONNECTION_DISCONNECTED;
		}
		
		///////////////////////////////////////////////////////////////////////////
		
		public Int32 CloseChannel(Int32 ConnectionHandle, Int32 ChannelHandle)
		{
			CloseChannelDelegate DCloseChannel = Connection.CloseChannel;
			
			// Set up the versioned hashtable input parameters
			Hashtable InParameters = new Hashtable();
			InParameters["Version"] = ESwarmVersionValue.VER_1_0;
			InParameters["ChannelHandle"] = ChannelHandle;
			
			// Invoke the method, then wait for it to finish or to be notified that the connection dropped
			Hashtable OutParameters = null;
			IAsyncResult Result = DCloseChannel.BeginInvoke(ConnectionHandle, InParameters, ref OutParameters, null, null);
			WaitHandle.WaitAny(new WaitHandle[]{ Result.AsyncWaitHandle, ConnectionDroppedEvent });
			
			// If the method completed normally, return the result
			if (Result.IsCompleted)
			{
				// If the invocation completed, success
				return DCloseChannel.EndInvoke(ref OutParameters, Result);
			}
			// Otherwise, error
			return Constants.ERROR_CONNECTION_DISCONNECTED;
		}
		
		///////////////////////////////////////////////////////////////////////////
		
		public Int32 OpenJob(Int32 ConnectionHandle, AgentGuid JobGuid )
		{
			OpenJobDelegate DOpenJob = Connection.OpenJob;
			
			// Set up the versioned hashtable input parameters
			Hashtable InParameters = new Hashtable();
			InParameters["Version"] = ESwarmVersionValue.VER_1_0;
			InParameters["JobGuid"] = JobGuid;
			
			// Invoke the method, then wait for it to finish or to be notified that the connection dropped
			Hashtable OutParameters = null;
			IAsyncResult Result = DOpenJob.BeginInvoke(ConnectionHandle, InParameters, ref OutParameters, null, null);
			WaitHandle.WaitAny(new WaitHandle[]{ Result.AsyncWaitHandle, ConnectionDroppedEvent });
			
			// If the method completed normally, return the result
			if (Result.IsCompleted)
			{
				// If the invocation completed, success
				return DOpenJob.EndInvoke(ref OutParameters, Result);
			}
			// Otherwise, error
			return Constants.ERROR_CONNECTION_DISCONNECTED;
		}
		
		///////////////////////////////////////////////////////////////////////////
		
		public Int32 BeginJobSpecification(Int32 ConnectionHandle, AgentJobSpecification Specification32, Hashtable Description32, AgentJobSpecification Specification64, Hashtable Description64)
		{
			BeginJobSpecificationDelegate DBeginJobSpecification = Connection.BeginJobSpecification;
			
			// Set up the versioned hashtable input parameters
			Hashtable InParameters = new Hashtable();
			InParameters["Version"] = ESwarmVersionValue.VER_1_0;
			InParameters["Specification32"] = Specification32;
			InParameters["Specification64"] = Specification64;
			InParameters["Description32"] = Description32;
			InParameters["Description64"] = Description64;
			
			// Invoke the method, then wait for it to finish or to be notified that the connection dropped
			Hashtable OutParameters = null;
			IAsyncResult Result = DBeginJobSpecification.BeginInvoke(ConnectionHandle, InParameters, ref OutParameters, null, null);
			WaitHandle.WaitAny(new WaitHandle[]{ Result.AsyncWaitHandle, ConnectionDroppedEvent });
			
			// If the method completed normally, return the result
			if (Result.IsCompleted)
			{
				// If the invocation completed, success
				return DBeginJobSpecification.EndInvoke(ref OutParameters, Result);
			}
			// Otherwise, error
			return Constants.ERROR_CONNECTION_DISCONNECTED;
		}
		
		///////////////////////////////////////////////////////////////////////////
		
		public Int32 AddTask(Int32 ConnectionHandle, List<AgentTaskSpecification> Specifications)
		{
			AddTaskDelegate DAddTask = Connection.AddTask;
			
			// Set up the versioned hashtable input parameters
			Hashtable InParameters = new Hashtable();
			InParameters["Version"] = ESwarmVersionValue.VER_1_0;
			InParameters["Specifications"] = Specifications;
			
			// Invoke the method, then wait for it to finish or to be notified that the connection dropped
			Hashtable OutParameters = null;
			IAsyncResult Result = DAddTask.BeginInvoke(ConnectionHandle, InParameters, ref OutParameters, null, null);
			WaitHandle.WaitAny(new WaitHandle[]{ Result.AsyncWaitHandle, ConnectionDroppedEvent });
			
			// If the method completed normally, return the result
			if (Result.IsCompleted)
			{
				// If the invocation completed, success
				return DAddTask.EndInvoke(ref OutParameters, Result);
			}
			// Otherwise, error
			return Constants.ERROR_CONNECTION_DISCONNECTED;
		}
		
		///////////////////////////////////////////////////////////////////////////
		
		public Int32 EndJobSpecification(Int32 ConnectionHandle)
		{
			EndJobSpecificationDelegate DEndJobSpecification = Connection.EndJobSpecification;
			
			// Invoke the method, then wait for it to finish or to be notified that the connection dropped
			Hashtable InParameters = null;
			Hashtable OutParameters = null;
			IAsyncResult Result = DEndJobSpecification.BeginInvoke(ConnectionHandle, InParameters, ref OutParameters, null, null);
			WaitHandle.WaitAny(new WaitHandle[]{ Result.AsyncWaitHandle, ConnectionDroppedEvent });
			
			// If the method completed normally, return the result
			if (Result.IsCompleted)
			{
				// If the invocation completed, success
				return DEndJobSpecification.EndInvoke(ref OutParameters, Result);
			}
			// Otherwise, error
			return Constants.ERROR_CONNECTION_DISCONNECTED;
		}
		
		///////////////////////////////////////////////////////////////////////////
		
		public Int32 CloseJob(Int32 ConnectionHandle)
		{
			CloseJobDelegate DCloseJob = Connection.CloseJob;
			
			// Invoke the method, then wait for it to finish or to be notified that the connection dropped
			Hashtable InParameters = null;
			Hashtable OutParameters = null;
			IAsyncResult Result = DCloseJob.BeginInvoke(ConnectionHandle, InParameters, ref OutParameters, null, null);
			WaitHandle.WaitAny(new WaitHandle[]{ Result.AsyncWaitHandle, ConnectionDroppedEvent });
			
			// If the method completed normally, return the result
			if (Result.IsCompleted)
			{
				// If the invocation completed, success
				return DCloseJob.EndInvoke(ref OutParameters, Result);
			}
			// Otherwise, error
			return Constants.ERROR_CONNECTION_DISCONNECTED;
		}
		
		///////////////////////////////////////////////////////////////////////////
		
		public Int32 Method(Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters)
		{
			MethodDelegate DMethod = Connection.Method;
			
			// Invoke the method, then wait for it to finish or to be notified that the connection dropped
			IAsyncResult Result = DMethod.BeginInvoke(ConnectionHandle, InParameters, ref OutParameters, null, null);
			WaitHandle.WaitAny(new WaitHandle[]{ Result.AsyncWaitHandle, ConnectionDroppedEvent });
			
			// If the method completed normally, return the result
			if (Result.IsCompleted)
			{
				// If the invocation completed, success
				return DMethod.EndInvoke(ref OutParameters, Result);
			}
			// Otherwise, error
			return Constants.ERROR_CONNECTION_DISCONNECTED;
		}
		
		///////////////////////////////////////////////////////////////////////////
		
		public Int32 Log(EVerbosityLevel Verbosity, ELogColour TextColour, String Line)
		{
			LogDelegate DLog = Connection.Log;
			
			// Invoke the method, then wait for it to finish or to be notified that the connection dropped
			IAsyncResult Result = DLog.BeginInvoke(Verbosity, TextColour, Line, null, null);
			WaitHandle.WaitAny(new WaitHandle[]{ Result.AsyncWaitHandle, ConnectionDroppedEvent });
			
			// If the method completed normally, return the result
			if (Result.IsCompleted)
			{
				// If the invocation completed, success
				return DLog.EndInvoke(Result);
			}
			// Otherwise, error
			return Constants.ERROR_CONNECTION_DISCONNECTED;
		}

		///////////////////////////////////////////////////////////////////////////
		
		// The wrapped connection
		IAgentInterface		Connection;
		ManualResetEvent	ConnectionDroppedEvent;
		
		// Delegate type declarations
		delegate Int32 OpenConnectionDelegate(Hashtable InParameters, ref Hashtable OutParameters);
		delegate Int32 CloseConnectionDelegate(Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters);
		delegate Int32 SendMessageDelegate(Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters);
		delegate Int32 GetMessageDelegate(Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters);
		delegate Int32 AddChannelDelegate(Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters);
		delegate Int32 TestChannelDelegate(Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters);
		delegate Int32 OpenChannelDelegate(Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters);
		delegate Int32 CloseChannelDelegate(Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters);
		delegate Int32 OpenJobDelegate(Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters);
		delegate Int32 BeginJobSpecificationDelegate(Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters);
		delegate Int32 AddTaskDelegate(Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters);
		delegate Int32 EndJobSpecificationDelegate(Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters);
		delegate Int32 CloseJobDelegate(Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters);
		delegate Int32 MethodDelegate(Int32 ConnectionHandle, Hashtable InParameters, ref Hashtable OutParameters);
		delegate Int32 LogDelegate(EVerbosityLevel Verbosity, ELogColour TextColour, String Line);
	}

	///////////////////////////////////////////////////////////////////////////////

	public delegate void FConnectionCallback(IntPtr CallbackMessage, IntPtr CallbackData);

	/**
	 * Helper struct for getting the message processing thread started
	 */
	public struct MessageThreadData
	{
		public FSwarmInterface				Owner;
		public IAgentInterfaceWrapper		Connection;
		public Int32						ConnectionHandle;
		public FConnectionCallback			ConnectionCallback;
		public IntPtr						ConnectionCallbackData;
		public AgentConfiguration			ConnectionConfiguration;
	}

	/**
	 * The C#implementation of FSwarmInterface
	 */
	public class FSwarmInterface
	{
		FSwarmInterface()
		{
			AgentProcess = null;
			AgentProcessOwner = false;
			Connection = null;
			ConnectionHandle = Constants.INVALID;
			ConnectionMessageThread = null;
			ConnectionMonitorThread = null;
			ConnectionConfiguration = null;
			ConnectionCallback = null;
			BaseChannelHandle = 0;
			PendingTasks = null;
			NetworkChannel = null;
			PerfTimerInstance = null;

			OpenChannels = new ReaderWriterDictionary<Int32, ChannelInfo>();
			FreeChannelWriteBuffers = new Stack<byte[]>();
			CleanupClosedConnectionLock = new Object();

			// TODO: Delete old files
		}

		delegate int SwarmOpenConnectionProc(FConnectionCallback CallbackFunc, IntPtr CallbackData, ELogFlags LoggingFlags, IntPtr OptionsFolder);
		delegate int SwarmCloseConnectionProc();
		delegate int SwarmSendMessageProc(IntPtr Message);
		delegate int SwarmAddChannelProc(IntPtr FullPath, IntPtr ChannelName);
		delegate int SwarmTestChannelProc(IntPtr ChannelName);
		delegate int SwarmOpenChannelProc(IntPtr ChannelName, EChannelFlags ChannelFlags);
		delegate int SwarmCloseChannelProc(int Channel);
		delegate int SwarmWriteChannelProc(int Channel, IntPtr Data, int DataSize);
		delegate int SwarmReadChannelProc(int Channel, IntPtr Data, int DataSize);
		delegate int SwarmOpenJobProc(IntPtr JobGuid);
		delegate int SwarmBeginJobSpecificationProc(IntPtr Specification32, IntPtr Specification64);
		delegate int SwarmAddTaskProc(IntPtr Specification);
		delegate int SwarmEndJobSpecificationProc();
		delegate int SwarmCloseJobProc();
		delegate int SwarmLogProc(EVerbosityLevel Verbosity, ELogColour TextColour, IntPtr Message);
		delegate int SwarmInterfaceLogDelegate(EVerbosityLevel Verbosity, IntPtr Message);

		static SwarmInterfaceLogDelegate SwarmInterfaceLogCppProc;

		private delegate void RegisterSwarmOpenConnectionProc(SwarmOpenConnectionProc Proc);
		private delegate void RegisterSwarmCloseConnectionProc(SwarmCloseConnectionProc Proc);
		private delegate void RegisterSwarmSendMessageProc(SwarmSendMessageProc Proc);
		private delegate void RegisterSwarmAddChannelProc(SwarmAddChannelProc Proc);
		private delegate void RegisterSwarmTestChannelProc(SwarmTestChannelProc Proc);
		private delegate void RegisterSwarmOpenChannelProc(SwarmOpenChannelProc Proc);
		private delegate void RegisterSwarmCloseChannelProc(SwarmCloseChannelProc Proc);
		private delegate void RegisterSwarmWriteChannelProc(SwarmWriteChannelProc Proc);
		private delegate void RegisterSwarmReadChannelProc(SwarmReadChannelProc Proc);
		private delegate void RegisterSwarmOpenJobProc(SwarmOpenJobProc Proc);
		private delegate void RegisterSwarmBeginJobSpecificationProc(SwarmBeginJobSpecificationProc Proc);
		private delegate void RegisterSwarmAddTaskProc(SwarmAddTaskProc Proc);
		private delegate void RegisterSwarmEndJobSpecificationProc(SwarmEndJobSpecificationProc Proc);
		private delegate void RegisterSwarmCloseJobProc(SwarmCloseJobProc Proc);
		private delegate void RegisterSwarmLogProc(SwarmLogProc Proc);

		static int SwarmOpenConnection(FConnectionCallback CallbackFunc, IntPtr CallbackData, ELogFlags LoggingFlags, IntPtr OptionsFolder)
		{
			try
			{
				return GInstance.OpenConnection(CallbackFunc, CallbackData, (ELogFlags)LoggingFlags, FStringMarshaler.MarshalNativeToManaged(OptionsFolder));
			}
			catch(Exception Ex)
			{
				DebugLog.Write(Ex.Message + "\n" + Ex.ToString());
				return 0;
			}
		}

		static int SwarmCloseConnection()
		{
			try
			{
				return GInstance.CloseConnection();
			}
			catch(Exception Ex)
			{
				DebugLog.Write(Ex.Message + "\n" + Ex.ToString());
				return 0;
			}
		}

		static int SwarmSendMessage(IntPtr Message)
		{
			try
			{
				return GInstance.SendMessage(Message);
			}
			catch(Exception Ex)
			{
				DebugLog.Write(Ex.Message + "\n" + Ex.ToString());
				return 0;
			}
		}

		static int SwarmAddChannel(IntPtr FullPath, IntPtr ChannelName)
		{
			try
			{
				return GInstance.AddChannel(FStringMarshaler.MarshalNativeToManaged(FullPath), FStringMarshaler.MarshalNativeToManaged(ChannelName));
			}
			catch(Exception Ex)
			{
				DebugLog.Write(Ex.Message + "\n" + Ex.ToString());
				return 0;
			}
		}

		static int SwarmTestChannel(IntPtr ChannelName)
		{
			try
			{
				return GInstance.TestChannel(FStringMarshaler.MarshalNativeToManaged(ChannelName));
			}
			catch(Exception Ex)
			{
				DebugLog.Write(Ex.Message + "\n" + Ex.ToString());
				return 0;
			}
		}

		static int SwarmOpenChannel(IntPtr ChannelName, EChannelFlags ChannelFlags)
		{
			try
			{
				return GInstance.OpenChannel(FStringMarshaler.MarshalNativeToManaged(ChannelName), ChannelFlags);
			}
			catch(Exception Ex)
			{
				DebugLog.Write(Ex.Message + "\n" + Ex.ToString());
				return 0;
			}
		}

		static int SwarmCloseChannel(int Channel)
		{
			try
			{
				return GInstance.CloseChannel(Channel);
			}
			catch(Exception Ex)
			{
				DebugLog.Write(Ex.Message + "\n" + Ex.ToString());
				return 0;
			}
		}

		static int SwarmWriteChannel(int Channel, IntPtr Data, int DataSize)
		{
			try
			{
				return GInstance.WriteChannel(Channel, Data, DataSize);
			}
			catch(Exception Ex)
			{
				DebugLog.Write(Ex.Message + "\n" + Ex.ToString());
				return 0;
			}
		}

		static int SwarmReadChannel(int Channel, IntPtr Data, int DataSize)
		{
			try
			{
				return GInstance.ReadChannel(Channel, Data, DataSize);
			}
			catch(Exception Ex)
			{
				DebugLog.Write(Ex.Message + "\n" + Ex.ToString());
				return 0;
			}
		}

		static int SwarmOpenJob(IntPtr JobGuid)
		{
			try
			{
				return GInstance.OpenJob((FGuid)Marshal.PtrToStructure(JobGuid, typeof(FGuid)));
			}
			catch(Exception Ex)
			{
				DebugLog.Write(Ex.Message + "\n" + Ex.ToString());
				return 0;
			}
		}

		static FJobSpecification MarshalJobSpecification(IntPtr SpecificationPtr)
		{
			FJobSpecificationMarshalHelper Helper = (FJobSpecificationMarshalHelper)Marshal.PtrToStructure(SpecificationPtr, typeof(FJobSpecificationMarshalHelper));

			FJobSpecification Specification = new FJobSpecification();
			Specification.ExecutableName = FStringMarshaler.MarshalNativeToManaged(Helper.ExecutableName);
			Specification.Parameters = FStringMarshaler.MarshalNativeToManaged(Helper.Parameters);
			Specification.Flags = Helper.Flags;

			Specification.RequiredDependencyCount = Helper.RequiredDependencyCount;
			Specification.RequiredDependencies = new String[Specification.RequiredDependencyCount];
			for (UInt32 Index = 0; Index < Specification.RequiredDependencyCount; Index++)
			{
				Specification.RequiredDependencies[Index] = FStringMarshaler.MarshalNativeToManaged(Marshal.ReadIntPtr(Helper.RequiredDependencies, (Int32)Index * 8));
			}

			Specification.OptionalDependencyCount = Helper.OptionalDependencyCount;
			Specification.OptionalDependencies = new String[Specification.OptionalDependencyCount];
			for (UInt32 Index = 0; Index < Specification.OptionalDependencyCount; Index++)
			{
				Specification.OptionalDependencies[Index] = FStringMarshaler.MarshalNativeToManaged(Marshal.ReadIntPtr(Helper.OptionalDependencies, (Int32)Index * 8));
			}

			Specification.DescriptionCount = Helper.DescriptionCount;
			Specification.DescriptionKeys = new String[Specification.DescriptionCount];
			Specification.DescriptionValues = new String[Specification.DescriptionCount];
			for (UInt32 Index = 0; Index < Specification.DescriptionCount; Index++)
			{
				Specification.DescriptionKeys[Index] = FStringMarshaler.MarshalNativeToManaged(Marshal.ReadIntPtr(Helper.DescriptionKeys, (Int32)Index * 8));
				Specification.DescriptionValues[Index] = FStringMarshaler.MarshalNativeToManaged(Marshal.ReadIntPtr(Helper.DescriptionValues, (Int32)Index * 8));
			}

			return Specification;
		}

		static int SwarmBeginJobSpecification(IntPtr Specification32, IntPtr Specification64)
		{
			try
			{
				return GInstance.BeginJobSpecification(MarshalJobSpecification(Specification32), MarshalJobSpecification(Specification64));
			}
			catch(Exception Ex)
			{
				DebugLog.Write(Ex.Message + "\n" + Ex.ToString());
				return 0;
			}
		}

		static FTaskSpecification MarshalTaskSpecification(IntPtr SpecificationPtr)
		{
			FTaskSpecificationMarshalHelper Helper = (FTaskSpecificationMarshalHelper)Marshal.PtrToStructure(SpecificationPtr, typeof(FTaskSpecificationMarshalHelper));
			FTaskSpecification Specification = new FTaskSpecification(Helper.TaskGuid, FStringMarshaler.MarshalNativeToManaged(Helper.Parameters), Helper.Flags);
			Specification.Cost = Helper.Cost;
			Specification.DependencyCount = Helper.DependencyCount;
			Specification.Dependencies = new String[Specification.DependencyCount];
			for (UInt32 Index = 0; Index < Specification.DependencyCount; Index++)
			{
				Specification.Dependencies[Index] = FStringMarshaler.MarshalNativeToManaged(Marshal.ReadIntPtr(Helper.Dependencies, (Int32)Index * 8));
			}
			return Specification;
		}

		static int SwarmAddTask(IntPtr Specification)
		{
			try
			{
				return GInstance.AddTask(MarshalTaskSpecification(Specification));
			}
			catch(Exception Ex)
			{
				DebugLog.Write(Ex.Message + "\n" + Ex.ToString());
				return 0;
			}
		}

		static int SwarmEndJobSpecification()
		{
			try
			{
				return GInstance.EndJobSpecification();
			}
			catch(Exception Ex)
			{
				DebugLog.Write(Ex.Message + "\n" + Ex.ToString());
				return 0;
			}
		}

		static int SwarmCloseJob()
		{
			try
			{
				return GInstance.CloseJob();
			}
			catch(Exception Ex)
			{
				DebugLog.Write(Ex.Message + "\n" + Ex.ToString());
				return 0;
			}
		}

		static int SwarmLog(EVerbosityLevel Verbosity, ELogColour TextColour, IntPtr Message)
		{
			try
			{
				return GInstance.Log((EVerbosityLevel)Verbosity, (ELogColour)TextColour, FStringMarshaler.MarshalNativeToManaged(Message));
			}
			catch(Exception Ex)
			{
				DebugLog.Write(Ex.Message + "\n" + Ex.ToString());
				return 0;
			}
		}

		static FSwarmInterface GInstance = new FSwarmInterface();
		static SwarmOpenConnectionProc OpenConnectionProc = new SwarmOpenConnectionProc(SwarmOpenConnection);
		static SwarmCloseConnectionProc CloseConnectionProc = new SwarmCloseConnectionProc(SwarmCloseConnection);
		static SwarmSendMessageProc SendMessageProc = new SwarmSendMessageProc(SwarmSendMessage);
		static SwarmAddChannelProc AddChannelProc = new SwarmAddChannelProc(SwarmAddChannel);
		static SwarmTestChannelProc TestChannelProc = new SwarmTestChannelProc(SwarmTestChannel);
		static SwarmOpenChannelProc OpenChannelProc = new SwarmOpenChannelProc(SwarmOpenChannel);
		static SwarmCloseChannelProc CloseChannelProc = new SwarmCloseChannelProc(SwarmCloseChannel);
		static SwarmWriteChannelProc WriteChannelProc = new SwarmWriteChannelProc(SwarmWriteChannel);
		static SwarmReadChannelProc ReadChannelProc = new SwarmReadChannelProc(SwarmReadChannel);
		static SwarmOpenJobProc OpenJobProc = new SwarmOpenJobProc(SwarmOpenJob);
		static SwarmBeginJobSpecificationProc BeginJobSpecificationProc = new SwarmBeginJobSpecificationProc(SwarmBeginJobSpecification);
		static SwarmAddTaskProc AddTaskProc = new SwarmAddTaskProc(SwarmAddTask);
		static SwarmEndJobSpecificationProc EndJobSpecificationProc = new SwarmEndJobSpecificationProc(SwarmEndJobSpecification);
		static SwarmCloseJobProc CloseJobProc = new SwarmCloseJobProc(SwarmCloseJob);
		static SwarmLogProc LogProc = new SwarmLogProc(SwarmLog);
		static bool KillMonitorThread = false;

#if !__MonoCS__
		[DllImport("kernel32.dll", CharSet = CharSet.Auto, SetLastError = true)]
		private static extern IntPtr LoadLibrary(string name);
		[DllImport("kernel32.dll", CharSet = CharSet.Ansi, SetLastError = true)]
		private static extern IntPtr GetProcAddress(IntPtr hModule, string name);
#else
		[DllImport("dl", CharSet = CharSet.Auto, SetLastError = true)]
		private static extern IntPtr dlopen(string name, int flag);
		[DllImport("dl", CharSet = CharSet.Ansi, SetLastError = true)]
		private static extern IntPtr dlsym(int handle, string name);
#endif

		/**
		 */
		public static int InitCppBridgeCallbacks(String SwarmInterfaceDllName)
		{
#if !__MonoCS__
			IntPtr DllHandle = LoadLibrary(SwarmInterfaceDllName);
			if (DllHandle == IntPtr.Zero)
			{
				return Constants.ERROR_FILE_FOUND_NOT;
			}

			{
				IntPtr ProcAddress = GetProcAddress(DllHandle, "RegisterSwarmOpenConnectionProc");
				var Proc = (RegisterSwarmOpenConnectionProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmOpenConnectionProc));
				Proc(OpenConnectionProc);
			}

			{
				IntPtr ProcAddress = GetProcAddress(DllHandle, "RegisterSwarmCloseConnectionProc");
				var Proc = (RegisterSwarmCloseConnectionProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmCloseConnectionProc));
				Proc(CloseConnectionProc);
			}

			{
				IntPtr ProcAddress = GetProcAddress(DllHandle, "RegisterSwarmSendMessageProc");
				var Proc = (RegisterSwarmSendMessageProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmSendMessageProc));
				Proc(SendMessageProc);
			}

			{
				IntPtr ProcAddress = GetProcAddress(DllHandle, "RegisterSwarmAddChannelProc");
				var Proc = (RegisterSwarmAddChannelProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmAddChannelProc));
				Proc(AddChannelProc);
			}

			{
				IntPtr ProcAddress = GetProcAddress(DllHandle, "RegisterSwarmTestChannelProc");
				var Proc = (RegisterSwarmTestChannelProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmTestChannelProc));
				Proc(TestChannelProc);
			}

			{
				IntPtr ProcAddress = GetProcAddress(DllHandle, "RegisterSwarmOpenChannelProc");
				var Proc = (RegisterSwarmOpenChannelProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmOpenChannelProc));
				Proc(OpenChannelProc);
			}

			{
				IntPtr ProcAddress = GetProcAddress(DllHandle, "RegisterSwarmCloseChannelProc");
				var Proc = (RegisterSwarmCloseChannelProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmCloseChannelProc));
				Proc(CloseChannelProc);
			}

			{
				IntPtr ProcAddress = GetProcAddress(DllHandle, "RegisterSwarmWriteChannelProc");
				var Proc = (RegisterSwarmWriteChannelProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmWriteChannelProc));
				Proc(WriteChannelProc);
			}

			{
				IntPtr ProcAddress = GetProcAddress(DllHandle, "RegisterSwarmReadChannelProc");
				var Proc = (RegisterSwarmReadChannelProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmReadChannelProc));
				Proc(ReadChannelProc);
			}

			{
				IntPtr ProcAddress = GetProcAddress(DllHandle, "RegisterSwarmOpenJobProc");
				var Proc = (RegisterSwarmOpenJobProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmOpenJobProc));
				Proc(OpenJobProc);
			}

			{
				IntPtr ProcAddress = GetProcAddress(DllHandle, "RegisterSwarmBeginJobSpecificationProc");
				var Proc = (RegisterSwarmBeginJobSpecificationProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmBeginJobSpecificationProc));
				Proc(BeginJobSpecificationProc);
			}

			{
				IntPtr ProcAddress = GetProcAddress(DllHandle, "RegisterSwarmAddTaskProc");
				var Proc = (RegisterSwarmAddTaskProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmAddTaskProc));
				Proc(AddTaskProc);
			}

			{
				IntPtr ProcAddress = GetProcAddress(DllHandle, "RegisterSwarmEndJobSpecificationProc");
				var Proc = (RegisterSwarmEndJobSpecificationProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmEndJobSpecificationProc));
				Proc(EndJobSpecificationProc);
			}

			{
				IntPtr ProcAddress = GetProcAddress(DllHandle, "RegisterSwarmCloseJobProc");
				var Proc = (RegisterSwarmCloseJobProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmCloseJobProc));
				Proc(CloseJobProc);
			}

			{
				IntPtr ProcAddress = GetProcAddress(DllHandle, "RegisterSwarmLogProc");
				var Proc = (RegisterSwarmLogProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmLogProc));
				Proc(LogProc);
			}

			{
				IntPtr ProcAddress = GetProcAddress(DllHandle, "SwarmInterfaceLog");
				SwarmInterfaceLogCppProc = (SwarmInterfaceLogDelegate)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(SwarmInterfaceLogDelegate));
			}
#else
			IntPtr DllHandle = dlopen(SwarmInterfaceDllName, 9 /* RTLD_LAZY | RTLD_GLOBAL */);
			if (DllHandle == IntPtr.Zero)
			{
				return Constants.ERROR_FILE_FOUND_NOT;
			}
			
			{
				IntPtr ProcAddress = dlsym(-2 /* RTLD_DEFAULT */, "RegisterSwarmOpenConnectionProc");
				var Proc = (RegisterSwarmOpenConnectionProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmOpenConnectionProc));
				Proc(SwarmOpenConnection);
			}
			
			{
				IntPtr ProcAddress = dlsym(-2 /* RTLD_DEFAULT */, "RegisterSwarmCloseConnectionProc");
				var Proc = (RegisterSwarmCloseConnectionProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmCloseConnectionProc));
				Proc(SwarmCloseConnection);
			}
			
			{
				IntPtr ProcAddress = dlsym(-2 /* RTLD_DEFAULT */, "RegisterSwarmSendMessageProc");
				var Proc = (RegisterSwarmSendMessageProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmSendMessageProc));
				Proc(SwarmSendMessage);
			}
			
			{
				IntPtr ProcAddress = dlsym(-2 /* RTLD_DEFAULT */, "RegisterSwarmAddChannelProc");
				var Proc = (RegisterSwarmAddChannelProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmAddChannelProc));
				Proc(SwarmAddChannel);
			}
			
			{
				IntPtr ProcAddress = dlsym(-2 /* RTLD_DEFAULT */, "RegisterSwarmTestChannelProc");
				var Proc = (RegisterSwarmTestChannelProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmTestChannelProc));
				Proc(SwarmTestChannel);
			}
			
			{
				IntPtr ProcAddress = dlsym(-2 /* RTLD_DEFAULT */, "RegisterSwarmOpenChannelProc");
				var Proc = (RegisterSwarmOpenChannelProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmOpenChannelProc));
				Proc(SwarmOpenChannel);
			}
			
			{
				IntPtr ProcAddress = dlsym(-2 /* RTLD_DEFAULT */, "RegisterSwarmCloseChannelProc");
				var Proc = (RegisterSwarmCloseChannelProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmCloseChannelProc));
				Proc(SwarmCloseChannel);
			}
			
			{
				IntPtr ProcAddress = dlsym(-2 /* RTLD_DEFAULT */, "RegisterSwarmWriteChannelProc");
				var Proc = (RegisterSwarmWriteChannelProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmWriteChannelProc));
				Proc(SwarmWriteChannel);
			}
			
			{
				IntPtr ProcAddress = dlsym(-2 /* RTLD_DEFAULT */, "RegisterSwarmReadChannelProc");
				var Proc = (RegisterSwarmReadChannelProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmReadChannelProc));
				Proc(SwarmReadChannel);
			}
			
			{
				IntPtr ProcAddress = dlsym(-2 /* RTLD_DEFAULT */, "RegisterSwarmOpenJobProc");
				var Proc = (RegisterSwarmOpenJobProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmOpenJobProc));
				Proc(SwarmOpenJob);
			}
			
			{
				IntPtr ProcAddress = dlsym(-2 /* RTLD_DEFAULT */, "RegisterSwarmBeginJobSpecificationProc");
				var Proc = (RegisterSwarmBeginJobSpecificationProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmBeginJobSpecificationProc));
				Proc(SwarmBeginJobSpecification);
			}
			
			{
				IntPtr ProcAddress = dlsym(-2 /* RTLD_DEFAULT */, "RegisterSwarmAddTaskProc");
				var Proc = (RegisterSwarmAddTaskProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmAddTaskProc));
				Proc(SwarmAddTask);
			}
			
			{
				IntPtr ProcAddress = dlsym(-2 /* RTLD_DEFAULT */, "RegisterSwarmEndJobSpecificationProc");
				var Proc = (RegisterSwarmEndJobSpecificationProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmEndJobSpecificationProc));
				Proc(SwarmEndJobSpecification);
			}
			
			{
				IntPtr ProcAddress = dlsym(-2 /* RTLD_DEFAULT */, "RegisterSwarmCloseJobProc");
				var Proc = (RegisterSwarmCloseJobProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmCloseJobProc));
				Proc(SwarmCloseJob);
			}
			
			{
				IntPtr ProcAddress = dlsym(-2 /* RTLD_DEFAULT */, "RegisterSwarmLogProc");
				var Proc = (RegisterSwarmLogProc)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(RegisterSwarmLogProc));
				Proc(SwarmLog);
			}

			{
				IntPtr ProcAddress = dlsym(-2 /* RTLD_DEFAULT */, "SwarmInterfaceLog");
				SwarmInterfaceLogCppProc = (SwarmInterfaceLogDelegate)Marshal.GetDelegateForFunctionPointer(ProcAddress, typeof(SwarmInterfaceLogDelegate));
			}
#endif

			return Constants.SUCCESS;
		}

		/**
		 * Opens a new connection to the Swarm
		 *
		 * @param CallbackFunc The callback function Swarm will use to communicate back to the Instigator
		 *
		 * @return An INT containing the error code (if < 0) or the handle (>= 0) which is useful for debugging only
		 */
		public virtual Int32 OpenConnection(FConnectionCallback CallbackFunc, IntPtr CallbackData, ELogFlags LoggingFlags, string OptionsFolder)
		{
			// Checked here so we can time OpenConnection
			if ((LoggingFlags & ELogFlags.LOG_TIMINGS) == ELogFlags.LOG_TIMINGS)
			{
				PerfTimerInstance = new PerfTimer();
			}

			StartTiming("OpenConnection-Managed", true);

			// Establish a connection to the local Agent server object
			ConnectionHandle = Constants.INVALID;
			Int32 ReturnValue = Constants.INVALID;
			try
			{
				EditorLog(EVerbosityLevel.Informative, "[OpenConnection] Registering TCP channel ...");

				// Start up network services, by opening a network communication channel
				NetworkChannel = new TcpClientChannel();
				ChannelServices.RegisterChannel(NetworkChannel, false);

				// See if an agent is already running, and if not, launch one
				EnsureAgentIsRunning(OptionsFolder);
				if (AgentProcess != null)
				{
					EditorLog(EVerbosityLevel.Informative, "[OpenConnection] Connecting to agent ...");
					ReturnValue = TryOpenConnection(CallbackFunc, CallbackData, LoggingFlags);
					if (ReturnValue >= 0)
					{
						AgentCacheFolder = ConnectionConfiguration.AgentCachePath;
						if (AgentCacheFolder.Length == 0)
						{
							EditorLog(EVerbosityLevel.Critical, "[OpenConnection] Agent cache folder with 0 length.");
							CloseConnection();
							ReturnValue = Constants.ERROR_FILE_FOUND_NOT;
						}
					}
				}
				else
				{
					EditorLog(EVerbosityLevel.Critical, "[OpenConnection] Failed to find Swarm Agent");
                    ReturnValue = Constants.ERROR_FILE_FOUND_NOT;
				}
			}
			catch (Exception Ex)
			{
				EditorLog(EVerbosityLevel.Critical, "[OpenConnection] Error: " + Ex.Message);
				ReturnValue = Constants.ERROR_EXCEPTION;
			}

			// Finally, if there have been no errors, assign the connection handle 
			if (ReturnValue >= 0)
			{
				ConnectionHandle = ReturnValue;
			}
			else
			{
				// If we've failed for any reason, call the clean up routine
				CleanupClosedConnection();
			}

			StopTiming();
			return ReturnValue;
		}

		/**
		 * Closes an existing connection to the Swarm
		 *
		 * @return Int32 error code (< 0 is error)
		 */
		public virtual Int32 CloseConnection()
		{
			// Dump any collected timing info
			DumpTimings();

			// Close the connection
			StartTiming("CloseConnection-Managed", true);

			Int32 ConnectionState = Constants.INVALID;
			if (Connection != null)
			{
				try
				{
					StartTiming("CloseConnection-Remote", false);
					Connection.CloseConnection(ConnectionHandle);
					StopTiming();

					ConnectionState = Constants.SUCCESS;
				}
				catch (Exception Ex)
				{
					ConnectionState = Constants.ERROR_EXCEPTION;
					DebugLog.Write("[CloseConnection] Error: " + Ex.Message);
				}

				// Clean up the state of the object with the connection now closed
				CleanupClosedConnection();

				// With the connecton completely closed, clean up our threads
				if (ConnectionMessageThread.Join(1000) == false)
				{
					// After calling CloseConnection, this thread is fair game to kill at any time
					Debug.WriteLineIf( Debugger.IsAttached, "[CloseConnection] Error: Message queue thread failed to quit before timeout, killing.");
					ConnectionMessageThread.Abort();
				}
				ConnectionMessageThread = null;
				KillMonitorThread = true;
				if (ConnectionMonitorThread.Join(1000) == false)
				{
					// We expect to abort this thread, no message necessary
					ConnectionMonitorThread.Abort();
				}
				ConnectionMonitorThread = null;
			}
			else
			{
				ConnectionState = Constants.ERROR_CONNECTION_NOT_FOUND;
			}

			StopTiming();
			return( ConnectionState );
		}

		/**
		 * Sends a message to an Agent (return messages are sent via the FConnectionCallback)
		 *
		 * @param Message The message being sent
		 *
		 * @return Int32 error code (< 0 is error)
		 */
		public virtual Int32 SendMessage(IntPtr NativeMessagePtr)
		{
			StartTiming("SendMessage-Managed", true);

			FMessage NativeMessage = (FMessage)Marshal.PtrToStructure(NativeMessagePtr, typeof(FMessage));

			Int32 ReturnValue = Constants.INVALID;
			if (Connection != null)
			{
				AgentMessage ManagedMessage = null;

				// TODO: As we add additional versions, convert to a switch rather than if-else.
				// For now, just use a simple if since we only have one version and a switch is
				// overkill.
				if (NativeMessage.Version == ESwarmVersionValue.VER_1_0)
				{
					switch (NativeMessage.Type)
					{
						case EMessageType.TASK_REQUEST_RESPONSE:
							// Swallow this message, since it should not be sent along to a local connection
							// since all Job and Task information is contained within the Agent itself
						break;

						case EMessageType.TASK_STATE:
						{
							FTaskState NativeTaskStateMessage = (FTaskState)Marshal.PtrToStructure(NativeMessagePtr, typeof(FTaskState));
							AgentGuid ManagedTaskGuid = new AgentGuid(NativeTaskStateMessage.TaskGuid.A,
																	  NativeTaskStateMessage.TaskGuid.B,
																	  NativeTaskStateMessage.TaskGuid.C,
																	  NativeTaskStateMessage.TaskGuid.D);
							EJobTaskState TaskState = (EJobTaskState)NativeTaskStateMessage.TaskState;
							AgentTaskState ManagedTaskStateMessage = new AgentTaskState(null, ManagedTaskGuid, TaskState);

							ManagedTaskStateMessage.TaskExitCode = NativeTaskStateMessage.TaskExitCode;
							ManagedTaskStateMessage.TaskRunningTime = NativeTaskStateMessage.TaskRunningTime;

							// If there is a message, be sure copy and pass it on
							if (NativeTaskStateMessage.TaskMessage != IntPtr.Zero)
							{
								ManagedTaskStateMessage.TaskMessage = FStringMarshaler.MarshalNativeToManaged(NativeTaskStateMessage.TaskMessage);
							}

							ManagedMessage = ManagedTaskStateMessage;
						}
						break;

						case EMessageType.INFO:
						{
							// Create the managed version of the info message
							FInfoMessage NativeInfoMessage = (FInfoMessage)Marshal.PtrToStructure(NativeMessagePtr, typeof(FInfoMessage));
							AgentInfoMessage ManagedInfoMessage = new AgentInfoMessage();
							if (NativeInfoMessage.TextMessage != IntPtr.Zero)
							{
								ManagedInfoMessage.TextMessage = FStringMarshaler.MarshalNativeToManaged(NativeInfoMessage.TextMessage);
							}
							ManagedMessage = ManagedInfoMessage;
						}
						break;

						case EMessageType.ALERT:
						{
							// Create the managed version of the alert message
							FAlertMessage NativeAlertMessage = (FAlertMessage)Marshal.PtrToStructure(NativeMessagePtr, typeof(FAlertMessage));
							AgentGuid JobGuid = new AgentGuid(NativeAlertMessage.JobGuid.A,
															  NativeAlertMessage.JobGuid.B,
															  NativeAlertMessage.JobGuid.C,
															  NativeAlertMessage.JobGuid.D);
							AgentAlertMessage ManagedAlertMessage = new AgentAlertMessage(JobGuid);
							ManagedAlertMessage.AlertLevel = (EAlertLevel)(NativeAlertMessage.AlertLevel);
							AgentGuid ObjectGuid = new AgentGuid(NativeAlertMessage.ObjectGuid.A,
																 NativeAlertMessage.ObjectGuid.B,
																 NativeAlertMessage.ObjectGuid.C,
																 NativeAlertMessage.ObjectGuid.D);
							ManagedAlertMessage.ObjectGuid = ObjectGuid;
							ManagedAlertMessage.TypeId = NativeAlertMessage.TypeId;
							if (NativeAlertMessage.TextMessage != IntPtr.Zero)
							{
								ManagedAlertMessage.TextMessage = FStringMarshaler.MarshalNativeToManaged(NativeAlertMessage.TextMessage);
							}
							ManagedMessage = ManagedAlertMessage;
						}
						break;

						case EMessageType.TIMING:
						{
							// Create the managed version of the info message
							FTimingMessage NativeTimingMessage = (FTimingMessage)Marshal.PtrToStructure(NativeMessagePtr, typeof(FTimingMessage));
							AgentTimingMessage ManagedTimingMessage = new AgentTimingMessage((EProgressionState)NativeTimingMessage.State, NativeTimingMessage.ThreadNum);
							ManagedMessage = ManagedTimingMessage;
						}
						break;

						default:
							// By default, just pass the message version and type through, but
							// any additional payload of a specialized type will be lost
							ManagedMessage = new AgentMessage((EMessageType)NativeMessage.Type);
						break;
					}
				}

				if (ManagedMessage != null)
				{
					try
					{
						// Finally, send the message to the Agent
						StartTiming("SendMessage-Remote", false);
						Connection.SendMessage(ConnectionHandle, ManagedMessage);
						StopTiming();
						ReturnValue = Constants.SUCCESS;
					}
					catch (Exception Ex)
					{
						Log(EVerbosityLevel.Critical, ELogColour.Red, "[Interface:SendMessage] Error: " + Ex.Message);
						ReturnValue = Constants.ERROR_CONNECTION_DISCONNECTED;
						CleanupClosedConnection();
					}
				}
			}
			else
			{
				ReturnValue = Constants.ERROR_CONNECTION_NOT_FOUND;
			}

			StopTiming();
			return( ReturnValue );
		}

		/**
		 * Adds an existing file to the cache. Note, any existing channel with the same
		 * name will be overwritten.
		 *
		 * @param FullPath The full path name to the file that should be copied into the cache
		 * @param ChannelName The name of the channel once it's in the cache
		 *
		 * @return Int32 error code (< 0 is error)
		 */
		public virtual Int32 AddChannel(String FullPath, String ChannelName)
		{
			StartTiming("AddChannel-Managed", true);

			Int32 ReturnValue = Constants.INVALID;
			if (Connection != null)
			{
				try
				{
					StartTiming("AddChannel-Remote", false);
					ReturnValue = Connection.AddChannel(ConnectionHandle, FullPath, ChannelName);
					StopTiming();
				}
				catch (Exception Ex)
				{
					Log(EVerbosityLevel.Critical, ELogColour.Red, "[Interface:AddChannel] Error: " + Ex.Message);
					ReturnValue = Constants.ERROR_CONNECTION_DISCONNECTED;
					CleanupClosedConnection();
				}
			}
			else
			{
				ReturnValue = Constants.ERROR_CONNECTION_NOT_FOUND;
			}

			StopTiming();
			return( ReturnValue );
		}

		/**
		 * Determines if the named channel is in the cache
		 *
		 * @param ChannelName The name of the channel to look for
		 *
		 * @return Int32 error code (< 0 is error)
		 */
		public virtual Int32 TestChannel(String ChannelName)
		{
			StartTiming("TestChannel-Managed", true);

			Int32 ReturnValue = Constants.INVALID;
			if (Connection != null)
			{
				try
				{
					if (!ConnectionConfiguration.IsPureLocalConnection)
					{
						StartTiming("TestChannel-Remote", false);
						ReturnValue = Connection.TestChannel(ConnectionHandle, ChannelName);
						StopTiming();
					}
					else
					{
						// Testing a channel only tests the main, persistent cache for files to read
						EChannelFlags ChannelFlags = (EChannelFlags)(EChannelFlags.TYPE_PERSISTENT | EChannelFlags.ACCESS_READ);
						String FullManagedName = GenerateFullChannelName(ChannelName, ChannelFlags);

						if (File.Exists(FullManagedName))
						{
							ReturnValue = Constants.SUCCESS;
						}
						else
						{
							ReturnValue = Constants.ERROR_FILE_FOUND_NOT;
						}
					}
				}
				catch (Exception Ex)
				{
					Log(EVerbosityLevel.Critical, ELogColour.Red, "[Interface:TestChannel] Error: " + Ex.Message);
					ReturnValue = Constants.ERROR_CONNECTION_DISCONNECTED;
					CleanupClosedConnection();
				}
			}
			else
			{
				ReturnValue = Constants.ERROR_CONNECTION_NOT_FOUND;
			}

			StopTiming();
			return( ReturnValue );
		}

		/**
		 * Opens a data channel for streaming data into the cache associated with an Agent
		 *
		 * @param ChannelName The name of the channel being opened
		 * @param ChannelFlags The mode, access, and other attributes of the channel being opened
		 *
		 * @return A handle to the opened channel (< 0 is error)
		 */
		public virtual Int32 OpenChannel(String ChannelName, EChannelFlags ChannelFlags)
		{
			StartTiming("OpenChannel-Managed", true);

			Int32 ChannelHandle = Constants.INVALID;
			bool NewChannelSuccessfullyCreated = false;
			if (Connection != null)
			{
				try
				{
					// Ask the Agent if the file is safe to open, if required
					if (!ConnectionConfiguration.IsPureLocalConnection)
					{
						StartTiming("OpenChannel-Remote", false);
						ChannelHandle = Connection.OpenChannel(ConnectionHandle, ChannelName, (EChannelFlags)ChannelFlags);
						StopTiming();
					}
					else
					{
						// If this is a pure local connection, then all we need to assure
						// if that the handle we generate here is unique to the connection
						ChannelHandle = Interlocked.Increment(ref BaseChannelHandle);
					}

					// If the channel is safe to open, open it up
					if (ChannelHandle >= 0)
					{
						// Track the newly created temp file
						ChannelInfo NewChannelInfo = new ChannelInfo();
						NewChannelInfo.ChannelName = ChannelName;
						NewChannelInfo.ChannelFlags = ChannelFlags;
						NewChannelInfo.ChannelHandle = ChannelHandle;
						NewChannelInfo.ChannelFileStream = null;
						NewChannelInfo.ChannelData = null;
						NewChannelInfo.ChannelOffset = 0;

						// Determine the proper path name for the file
						String FullManagedName = GenerateFullChannelName(ChannelName, ChannelFlags);
				
						// Try to open the file
						if ((ChannelFlags & EChannelFlags.ACCESS_WRITE) != 0)
						{
							// Open the file stream
							NewChannelInfo.ChannelFileStream = new FileStream(FullManagedName, FileMode.Create, FileAccess.Write, FileShare.None);

							// Slightly different path for compressed files
							if ((ChannelFlags & EChannelFlags.MISC_ENABLE_COMPRESSION) != 0)
							{
								Stream NewChannelStream = NewChannelInfo.ChannelFileStream;
								NewChannelInfo.ChannelFileStream = new GZipStream(NewChannelStream, CompressionMode.Compress, false);
							}
					
							// If we were able to open the file, add it to the active channel list
							Monitor.Enter(FreeChannelWriteBuffers);
							try
							{
								// If available, take the next free write buffer from the list
								if (FreeChannelWriteBuffers.Count > 0)
								{
									NewChannelInfo.ChannelData = FreeChannelWriteBuffers.Pop();
								}
								else
								{
									// Otherwise, allocate a new write buffer for this channel (default to 1MB)
									NewChannelInfo.ChannelData = new byte[1024 * 1024];
								}
							}
							finally
							{
								Monitor.Exit( FreeChannelWriteBuffers );
							}

							// Track the newly created file
							OpenChannels.Add(ChannelHandle, NewChannelInfo);
							NewChannelSuccessfullyCreated = true;
						}
						else if ((ChannelFlags & EChannelFlags.ACCESS_READ) != 0)
						{
							if (File.Exists(FullManagedName))
							{
								// Slightly different paths for compressed and non-compressed files
								if ((ChannelFlags & EChannelFlags.MISC_ENABLE_COMPRESSION) != 0)
								{
									// Open the input stream, loading it entirely into memory
									byte[] RawCompressedData = File.ReadAllBytes(FullManagedName);

									// Allocate the destination buffer
									// The size of the uncompressed data is contained in the last four bytes of the file
									// http://www.ietf.org/rfc/rfc1952.txt?number=1952
									Int32 UncompressedSize = BitConverter.ToInt32(RawCompressedData, RawCompressedData.Length - 4);
									NewChannelInfo.ChannelData = new byte[UncompressedSize];

									// Open the decompression stream and decompress directly into the destination
									Stream DecompressionChannelStream = new GZipStream(new MemoryStream(RawCompressedData), CompressionMode.Decompress, false);
									DecompressionChannelStream.Read(NewChannelInfo.ChannelData, 0, UncompressedSize);
									DecompressionChannelStream.Close();
								}
								else
								{
									// Simply read in the entire file in one go
									NewChannelInfo.ChannelData = File.ReadAllBytes(FullManagedName);
								}

								// Track the newly created channel
								OpenChannels.Add(ChannelHandle, NewChannelInfo);
								NewChannelSuccessfullyCreated = true;
							}
							else
							{
								// Failed to find the channel to read, return an error
								ChannelHandle = Constants.ERROR_CHANNEL_NOT_FOUND;
							}
						}
					}
				}
				catch (Exception Ex)
				{
					Log(EVerbosityLevel.Critical, ELogColour.Red, "[Interface:OpenChannel] Error: " + Ex.ToString());
					ChannelHandle = Constants.ERROR_CONNECTION_DISCONNECTED;
					CleanupClosedConnection();
				}

				// If we opened the channel on the agent, but failed to create
				// the file, close it on the agent
				if ((ChannelHandle >= 0) &&
					(NewChannelSuccessfullyCreated == false))
				{
					if (!ConnectionConfiguration.IsPureLocalConnection)
					{
						StartTiming("CloseChannel-Remote", false);
						try
						{
							Connection.CloseChannel(ConnectionHandle, ChannelHandle);
						}
						catch (Exception Ex)
						{
							Log(EVerbosityLevel.Critical, ELogColour.Red, "[Interface:OpenChannel] Cleanup error: " + Ex.Message);
							CleanupClosedConnection();
						}
						StopTiming();
					}
					ChannelHandle = Constants.ERROR_CHANNEL_IO_FAILED;
				}
			}
			else
			{
				ChannelHandle = Constants.ERROR_CONNECTION_NOT_FOUND;
			}

			StopTiming();
			return ChannelHandle;
		}

		/**
		 * Closes an open channel
		 *
		 * @param Channel An open channel handle, returned by OpenChannel
		 *
		 * @return Int32 error code (< 0 is error)
		 */
		public virtual Int32 CloseChannel(Int32 Channel)
		{
			StartTiming("CloseChannel-Managed", true);

			Int32 ReturnValue = Constants.INVALID;
			if (Connection != null)
			{
				// Get the channel info, so we can close the connection on the Agent
				ChannelInfo ChannelToClose = null;
				if (OpenChannels.TryGetValue(Channel, out ChannelToClose))
				{
					try
					{
						// If the channel was open for WRITE, make sure any buffers are flushed
						if ((ChannelToClose.ChannelFlags & EChannelFlags.ACCESS_WRITE) != 0)
						{
							FlushChannel(ChannelToClose);

							// Now that we're done with the write buffer, add it to the free stack
							// for another channel to use (only for WRITE)
							Monitor.Enter(FreeChannelWriteBuffers);
							try
							{
								FreeChannelWriteBuffers.Push(ChannelToClose.ChannelData);
							}
							finally
							{
								Monitor.Exit(FreeChannelWriteBuffers);
							}
						}

						// Remove the channel from the collection of open channels for this connection
						OpenChannels.Remove(Channel);

						// Close the file handle
						if (ChannelToClose.ChannelFileStream != null)
						{
							ChannelToClose.ChannelFileStream.Close();
							ChannelToClose.ChannelFileStream = null;
						}

						// Notify the Agent that the channel is closed, if required
						if (!ConnectionConfiguration.IsPureLocalConnection)
						{
							StartTiming("CloseChannel-Remote", false);
							Connection.CloseChannel(ConnectionHandle, ChannelToClose.ChannelHandle);
							StopTiming();
						}
						else
						{
							// Since this is a pure local connection, all we need to do is make
							// sure the now-closed channel is moved from the staging area into
							// the main cache, but only if the channel is PERSISTENT and WRITE
							if ((ChannelToClose.ChannelFlags & EChannelFlags.TYPE_PERSISTENT) != 0)
							{
								if ((ChannelToClose.ChannelFlags & EChannelFlags.ACCESS_WRITE) != 0)
								{
									// Get the final location path
									EChannelFlags WriteChannelFlags = (EChannelFlags)(EChannelFlags.TYPE_PERSISTENT | EChannelFlags.ACCESS_WRITE);
									String SrcChannelName = GenerateFullChannelName(ChannelToClose.ChannelName, WriteChannelFlags);
									EChannelFlags ReadChannelFlags = (EChannelFlags)(EChannelFlags.TYPE_PERSISTENT | EChannelFlags.ACCESS_READ);
									String DstChannelName = GenerateFullChannelName(ChannelToClose.ChannelName, ReadChannelFlags);

									// Always remove the destination file if it already exists
									FileInfo DstChannel = new FileInfo(DstChannelName);
									if (DstChannel.Exists)
									{
										DstChannel.IsReadOnly = false;
										DstChannel.Delete();
									}

									// Copy if the paper trail is enabled; Move otherwise
									if ((ChannelToClose.ChannelFlags & EChannelFlags.MISC_ENABLE_PAPER_TRAIL) != 0)
									{
										File.Copy(SrcChannelName, DstChannelName);
									}
									else
									{
										File.Move(SrcChannelName, DstChannelName);
									}
								}
							}
						}

						ReturnValue = Constants.SUCCESS;
					}
					catch (Exception Ex)
					{
						Log(EVerbosityLevel.Critical, ELogColour.Red, "[Interface:CloseChannel] Error: " + Ex.Message);
						ReturnValue = Constants.ERROR_CONNECTION_DISCONNECTED;
						CleanupClosedConnection();
					}
				}
				else
				{
					ReturnValue = Constants.ERROR_CHANNEL_NOT_FOUND;
				}
			}
			else
			{
				ReturnValue = Constants.ERROR_CONNECTION_NOT_FOUND;
			}

			StopTiming();
			return ReturnValue;
		}

		/**
		 * Writes the provided data to the open channel opened for WRITE
		 *
		 * @param Channel An open channel handle, returned by OpenChannel
		 * @param Data Source buffer for the write
		 * @param Data Size of the source buffer
		 *
		 * @return The number of bytes written (< 0 is error)
		 */
		public virtual Int32 WriteChannel(Int32 Channel, IntPtr Data, Int32 DataSize)
		{
			StartTiming("WriteChannel-Managed", true, DataSize);

			Int32 ReturnValue = Constants.INVALID;
			if (Connection != null)
			{
				ChannelInfo ChannelToWrite = null;
				if ((OpenChannels.TryGetValue(Channel, out ChannelToWrite)) &&
					(ChannelToWrite.ChannelFileStream != null))
				{
					try
					{
						bool WriteBuffered = false;
						if (ChannelToWrite.ChannelData != null)
						{
							// See if the new data will fit into the write buffer
							if ((ChannelToWrite.ChannelOffset + DataSize) <= ChannelToWrite.ChannelData.Length)
							{
								Marshal.Copy(Data, ChannelToWrite.ChannelData, ChannelToWrite.ChannelOffset, DataSize);
								ChannelToWrite.ChannelOffset += DataSize;
								ReturnValue = DataSize;
								WriteBuffered = true;
							}
							else
							{
								// Otherwise, flush any pending writes and try again with the reset buffer
								FlushChannel(ChannelToWrite);
								if (DataSize <= ChannelToWrite.ChannelData.Length)
								{
									Marshal.Copy(Data, ChannelToWrite.ChannelData, 0, DataSize);
									ChannelToWrite.ChannelOffset = DataSize;
									ReturnValue = DataSize;
									WriteBuffered = true;
								}
							}
						}

						// Write was not buffered, just write it directly out
						if (!WriteBuffered)
						{
							try
							{
								// Allocate a temporary buffer and copy the data in
								byte[] TempChannelData = new byte[DataSize];
								Marshal.Copy(Data, TempChannelData, 0, DataSize);
								ChannelToWrite.ChannelFileStream.Write(TempChannelData, 0, DataSize);
								ReturnValue = DataSize;
							}
							catch (Exception Ex)
							{
								Log(EVerbosityLevel.Critical, ELogColour.Red, "[Interface:WriteChannel] Error: " + Ex.Message);
								ReturnValue = Constants.ERROR_CHANNEL_IO_FAILED;
							}
						}
					}
					catch (Exception Ex)
					{
						Log(EVerbosityLevel.Critical, ELogColour.Red, "[Interface:WriteChannel] Error: " + Ex.Message);
						ReturnValue = Constants.ERROR_CONNECTION_DISCONNECTED;
						CleanupClosedConnection();
					}

					// If this was not a successful IO operation, remove the channel
					// from the set of active channels
					if (ReturnValue < 0)
					{
						OpenChannels.Remove(Channel);

						try
						{
							ChannelToWrite.ChannelFileStream.Close();
							ChannelToWrite.ChannelFileStream = null;
						}
						catch (Exception Ex)
						{
							Log(EVerbosityLevel.Critical, ELogColour.Red, "[Interface:WriteChannel] Error: " + Ex.Message);
							ReturnValue = Constants.ERROR_CONNECTION_DISCONNECTED;
							CleanupClosedConnection();
						}
					}
				}
				else
				{
					ReturnValue = Constants.ERROR_CHANNEL_NOT_FOUND;
				}
			}
			else
			{
				ReturnValue = Constants.ERROR_CONNECTION_NOT_FOUND;
			}

			StopTiming();
			return ReturnValue;
		}

		/**
		 * Reads data from a channel opened for READ into the provided buffer
		 *
		 * @param Channel An open channel handle, returned by OpenChannel
		 * @param Data Destination buffer for the read
		 * @param Data Size of the destination buffer
		 *
		 * @return The number of bytes read (< 0 is error)
		 */
		public virtual Int32 ReadChannel(Int32 Channel, IntPtr Data, Int32 DataSize)
		{
			StartTiming("ReadChannel-Managed", true, DataSize);

			Int32 ReturnValue = Constants.INVALID;
			if (Connection != null)
			{
				ChannelInfo ChannelToRead = null;
				if ((OpenChannels.TryGetValue(Channel, out ChannelToRead)) &&
					(ChannelToRead.ChannelData != null))
				{
					try
					{
						// Read the data directly from our buffered copy of the file
						Int32 DataRemaining = ChannelToRead.ChannelData.Length - ChannelToRead.ChannelOffset;
						if (DataRemaining >= 0)
						{
							Int32 SizeToRead = DataRemaining < DataSize ? DataRemaining : DataSize;
							if (SizeToRead > 0)
							{
								Marshal.Copy(ChannelToRead.ChannelData, ChannelToRead.ChannelOffset, Data, SizeToRead);
								ChannelToRead.ChannelOffset += SizeToRead;
							}
							ReturnValue = SizeToRead;
						}
						else
						{
							ReturnValue = Constants.ERROR_CHANNEL_IO_FAILED;
						}
					}
					catch (Exception Ex)
					{
						Log(EVerbosityLevel.Critical, ELogColour.Red, "[Interface:ReadChannel] Error: " + Ex.Message);
						ReturnValue = Constants.ERROR_CONNECTION_DISCONNECTED;
						CleanupClosedConnection();
					}

					// If this was not a successful IO operation, remove the channel
					// from the set of active channels
					if (ReturnValue < 0)
					{
						OpenChannels.Remove(Channel);
					}
				}
				else
				{
					ReturnValue = Constants.ERROR_CHANNEL_NOT_FOUND;
				}
			}
			else
			{
				ReturnValue = Constants.ERROR_CONNECTION_NOT_FOUND;
			}

			StopTiming();
			return ReturnValue;
		}

		/**
		 * Opens a Job session, which allows a Job to be specified, Tasks added, Job
		 * channels opened and used, etc. When the Job is complete and no more Job
		 * related data is needed from the Swarm, call CloseJob.
		 *
		 * @param JobGuid A GUID that uniquely identifies this Job, generated by the caller
		 *
		 * @return Int32 Error code (< 0 is an error)
		 */
		public virtual Int32 OpenJob(FGuid JobGuid)
		{
			StartTiming("OpenJob-Managed", true);

			Int32 ReturnValue = Constants.INVALID;
			if (Connection != null)
			{
				StartTiming("OpenJob-Remote", false);
				try
				{
					AgentGuid ManagedJobGuid = new AgentGuid(JobGuid.A, JobGuid.B, JobGuid.C, JobGuid.D);
					ReturnValue = Connection.OpenJob(ConnectionHandle, ManagedJobGuid);
					if (ReturnValue >= 0)
					{
						// If the call was successful, assign the Job Guid as the active one
						ConnectionConfiguration.AgentJobGuid = ManagedJobGuid;

						// Allocate a new list to collect tasks until the specification is complete
						PendingTasks = new List<AgentTaskSpecification>();
					}
				}
				catch (Exception Ex)
				{
					Log(EVerbosityLevel.Critical, ELogColour.Red, "[Interface:OpenJob] Error: " + Ex.Message);
					ReturnValue = Constants.ERROR_CONNECTION_DISCONNECTED;
					CleanupClosedConnection();
				}
				StopTiming();
			}
			else
			{
				ReturnValue = Constants.ERROR_CONNECTION_NOT_FOUND;
			}

			StopTiming();
			return ReturnValue;
		}

		/**
		 * Begins a Job specification, which allows a series of Tasks to be specified
		 * via AddTask. When Tasks are done being specified, call EndJobSpecification.
		 *
		 * The default behavior will be to execute the Job executable with the
		 * specified parameters. If Tasks are added for the Job, they are expected
		 * to be requested by the executable run for the Job. If no Tasks are added
		 * for the Job, it is expected that the Job executable will perform its
		 * operations without additional Task input from Swarm.
		 *
		 * @param Specification32 A structure describing a new 32-bit Job (can be an empty specification)
		 * @param Specification64 A structure describing a new 64-bit Job (can be an empty specification)
		 *
		 * @return Int32 Error code (< 0 is an error)
		 */
		public virtual Int32 BeginJobSpecification(FJobSpecification Specification32, FJobSpecification Specification64)
		{
			StartTiming("BeginJobSpecification-Managed", true );

			Int32 ReturnValue = Constants.INVALID;
			if (Connection != null)
			{
				if (ConnectionConfiguration.AgentJobGuid != null)
				{
					// Convert the specifications from native to managed
					AgentJobSpecification NewSpecification32 = ConvertJobSpecification(ref Specification32, false);
					AgentJobSpecification NewSpecification64 = ConvertJobSpecification(ref Specification64, true);

					// Ensure all the files are in the cache with the right cache compatible name
					if (NewSpecification32 != null)
					{
						ReturnValue = CacheAllFiles(NewSpecification32);
					}
					if (NewSpecification64 != null && (NewSpecification32 == null || ReturnValue < 0))
					{
						ReturnValue = CacheAllFiles(NewSpecification64);
					}

					if (ReturnValue >= 0)
					{
						// Pack up the optional descriptions into Hashtables and pass them along
						UInt32 DescriptionIndex;
						Hashtable NewDescription32 = null;
						Hashtable NewDescription64 = null;
						// 32-bit specification description 
						if (Specification32.DescriptionCount > 0)
						{
							try
							{
								NewDescription32 = new Hashtable();
								NewDescription32["Version"] = ESwarmVersionValue.VER_1_0;
								for (DescriptionIndex = 0; DescriptionIndex < Specification32.DescriptionCount; DescriptionIndex++)
								{
									String NewKey = Specification32.DescriptionKeys[DescriptionIndex];
									String NewValue = Specification32.DescriptionValues[DescriptionIndex];
									NewDescription32[NewKey] = NewValue;
								}
							}
							catch (Exception Ex)
							{
								// Failed to transfer optional description, log and continue
								Log(EVerbosityLevel.Critical, ELogColour.Red, "[Interface:BeginJobSpecification] Error with Specification32 Description: " + Ex.Message);
								NewDescription32 = null;
							}
						}
						// 64-bit specification description 
						if (Specification64.DescriptionCount > 0)
						{
							try
							{
								NewDescription64 = new Hashtable();
								NewDescription64["Version"] = ESwarmVersionValue.VER_1_0;
								for (DescriptionIndex = 0; DescriptionIndex < Specification64.DescriptionCount; DescriptionIndex++)
								{
									String NewKey = Specification64.DescriptionKeys[DescriptionIndex];
									String NewValue = Specification64.DescriptionValues[DescriptionIndex];
									NewDescription64[NewKey] = NewValue;
								}
							}
							catch (Exception Ex)
							{
								// Failed to transfer optional description, log and continue
								Log(EVerbosityLevel.Critical, ELogColour.Red, "[Interface:BeginJobSpecification] Error with Specification64 Description: " + Ex.Message);
								NewDescription64 = null;
							}
						}

						StartTiming("BeginJobSpecification-Remote", false);
						try
						{
							ReturnValue = Connection.BeginJobSpecification(ConnectionHandle, NewSpecification32, NewDescription32, NewSpecification64, NewDescription64);
						}
						catch (Exception Ex)
						{
							Log(EVerbosityLevel.Critical, ELogColour.Red, "[Interface:BeginJobSpecification] Error: " + Ex.Message);
							ReturnValue = Constants.ERROR_CONNECTION_DISCONNECTED;
							CleanupClosedConnection();
						}
						StopTiming();
					}
				}
				else
				{
					ReturnValue = Constants.ERROR_JOB_NOT_FOUND;
				}
			}
			else
			{
				ReturnValue = Constants.ERROR_CONNECTION_NOT_FOUND;
			}

			StopTiming();
			return ReturnValue;
		}

		/**
		 * Adds a Task to the current Job
		 *
		 * @param Specification A structure describing the new Task
		 *
		 * @return Int32 Error code (< 0 is an error)
		 */
		public virtual Int32 AddTask(FTaskSpecification Specification)
		{
			StartTiming("AddTask-Managed", true);

			Int32 ReturnValue = Constants.INVALID;
			if (Connection != null)
			{
				if (ConnectionConfiguration.AgentJobGuid != null)
				{
					// Convert the parameters from native to managed
					AgentGuid TaskGuid = new AgentGuid(Specification.TaskGuid.A,
													   Specification.TaskGuid.B,
													   Specification.TaskGuid.C,
													   Specification.TaskGuid.D);

					String Parameters = Specification.Parameters;

					List<String> Dependencies = null;
					if (Specification.DependencyCount > 0)
					{
						Dependencies = new List<String>();
						for (UInt32 i = 0; i < Specification.DependencyCount; i++)
						{
							Dependencies.Add(Specification.Dependencies[i]);
						}
					}

					AgentTaskSpecification NewSpecification =
						new AgentTaskSpecification(ConnectionConfiguration.AgentJobGuid, TaskGuid, (Int32)Specification.Flags, Parameters, (Int32)Specification.Cost, Dependencies);

					// Ensure all the files are in the cache with the right cache compatible name
					ReturnValue = CacheAllFiles(NewSpecification);
					if (ReturnValue >= 0)
					{
						// Queue up all tasks until the specification is complete and submit them all at once
						PendingTasks.Add(NewSpecification);
					}
				}
				else
				{
					ReturnValue = Constants.ERROR_JOB_NOT_FOUND;
				}
			}
			else
			{
				ReturnValue = Constants.ERROR_CONNECTION_NOT_FOUND;
			}

			StopTiming();
			return ReturnValue;
		}

		/**
		 * Ends the Job specification, after which no additional Tasks may be defined. Also,
		 * this is generally the point when the Agent will validate and launch the Job executable,
		 * potentially distributing the Job to other Agents.
		 *
		 * @return Int32 Error code (< 0 is an error)
		 */
		public virtual Int32 EndJobSpecification()
		{
			StartTiming("EndJobSpecification-Managed", true);

			Int32 ReturnValue = Constants.INVALID;
			if (Connection != null)
			{
				if (ConnectionConfiguration.AgentJobGuid != null)
				{
					StartTiming("EndJobSpecification-Remote", false);
					try
					{
						// Add all queued up and pending tasks now, all at once
						ReturnValue = Connection.AddTask(ConnectionHandle, PendingTasks);
						if( ReturnValue >= 0 )
						{
							// Finally, end the specification
							ReturnValue = Connection.EndJobSpecification(ConnectionHandle);
						}
					}
					catch (Exception Ex)
					{
						Log(EVerbosityLevel.Critical, ELogColour.Red, "[Interface:EndJobSpecification] Error: " + Ex.Message + " " + Ex.ToString());
						ReturnValue = Constants.ERROR_CONNECTION_DISCONNECTED;
						CleanupClosedConnection();
					}
					PendingTasks = null;
					StopTiming();
				}
				else
				{
					ReturnValue = Constants.ERROR_JOB_NOT_FOUND;
				}
			}
			else
			{
				ReturnValue = Constants.ERROR_CONNECTION_NOT_FOUND;
			}

			StopTiming();
			return ReturnValue;
		}

		/**
		 * Ends the Job, after which all Job-related API usage (except OpenJob) will be rejected
		 *
		 * @return Int32 Error code (< 0 is an error)
		 */
		public virtual Int32 CloseJob()
		{
			StartTiming("CloseJob-Managed", true);

			Int32 ReturnValue = Constants.INVALID;
			if (Connection != null)
			{
				if (ConnectionConfiguration.AgentJobGuid != null)
				{
					StartTiming("CloseJob-Remote", false);
					try
					{
						ReturnValue = Connection.CloseJob(ConnectionHandle);
					}
					catch (Exception Ex)
					{
						Log(EVerbosityLevel.Critical, ELogColour.Red, "[Interface:CloseJob] Error: " + Ex.Message);
						ReturnValue = Constants.ERROR_CONNECTION_DISCONNECTED;
						CleanupClosedConnection();
					}
					StopTiming();

					// Reset the active Job Guid value
					try
					{
						ConnectionConfiguration.AgentJobGuid = null;
					}
					catch (Exception)
					{
						// The ConnectionConfiguration can be set to null
						// asynchronously, so we'll need to try/catch here
					}
				}
				else
				{
					ReturnValue = Constants.ERROR_JOB_NOT_FOUND;
				}
			}
			else
			{
				ReturnValue = Constants.ERROR_CONNECTION_NOT_FOUND;
			}

			StopTiming();
			return ReturnValue;
		}

		/**
		 * Adds a line of text to the Agent log window
		 *
		 * @param Verbosity the importance of this message
		 * @param TextColour the colour of the text
		 * @param Message the line of text to add
		 */
		public virtual Int32 Log(EVerbosityLevel Verbosity, ELogColour TextColour, String Message)
		{
			try
			{
				Connection.Log(Verbosity, TextColour, Message);
			}
			catch (Exception)
			{
			}

			return Constants.SUCCESS;
		}

		/**
		 * The function for the message queue monitoring thread used for
		 * calling the callback from the remote Agent
		 */
		static void MessageThreadProc(Object ThreadParameters)
		{
			MessageThreadData ThreadData = (MessageThreadData)ThreadParameters;
			IAgentInterfaceWrapper Connection = ThreadData.Connection;

			try
			{
				// Because the way we use the GetMessage call is blocking, if we ever break out, quit
				AgentMessage ManagedMessage = null;
				while (Connection.GetMessage(ThreadData.ConnectionHandle, out ManagedMessage, -1) >= 0)
				{
					// TODO: As we add additional versions, convert to a switch rather than if-else.
					// For now, just use a simple if since we only have one version and a switch is
					// overkill.
					if (ManagedMessage.Version == ESwarmVersionValue.VER_1_0)
					{
						IntPtr NativeMessage = IntPtr.Zero;
						String PinnedStringData = null;

						switch (ManagedMessage.Type)
						{
							case EMessageType.JOB_STATE:
							{
								AgentJobState JobStateMessage = (AgentJobState)ManagedMessage;
								FGuid JobGuid = new FGuid(JobStateMessage.JobGuid.A,
														  JobStateMessage.JobGuid.B,
														  JobStateMessage.JobGuid.C,
														  JobStateMessage.JobGuid.D);

								FJobState NativeJobStateMessage = new FJobState(JobGuid, (EJobTaskState)JobStateMessage.JobState);
								NativeJobStateMessage.JobExitCode = JobStateMessage.JobExitCode;
								NativeJobStateMessage.JobRunningTime = JobStateMessage.JobRunningTime;

								// If there is a message, be sure to pin and pass it on
								if (JobStateMessage.JobMessage != null)
								{
									Char[] RawJobMessageData = JobStateMessage.JobMessage.ToCharArray();
									if (RawJobMessageData.Length > 0)
									{
										// Pin the string data for the message
										PinnedStringData = new String(RawJobMessageData);
										NativeJobStateMessage.JobMessage = FStringMarshaler.MarshalManagedToNative(PinnedStringData);
									}
								}

								NativeMessage = Marshal.AllocHGlobal(Marshal.SizeOf(typeof(FJobState)));
								Marshal.StructureToPtr(NativeJobStateMessage, NativeMessage, false);
							}
							break;

							case EMessageType.TASK_REQUEST:
								// Swallow this message, since it should not be sent along to a local connection
								// since all Job and Task information is contained within the Agent itself
							break;

							case EMessageType.TASK_REQUEST_RESPONSE:
							{
								AgentTaskRequestResponse TaskRequestResponseMessage = (AgentTaskRequestResponse)ManagedMessage;

								// Switch again on the response type
								ETaskRequestResponseType ResponseType = (ETaskRequestResponseType)TaskRequestResponseMessage.ResponseType;
								switch (ResponseType)
								{
									case ETaskRequestResponseType.RELEASE:
									case ETaskRequestResponseType.RESERVATION:
									{
										NativeMessage = Marshal.AllocHGlobal(Marshal.SizeOf(typeof(FTaskRequestResponse)));
										Marshal.StructureToPtr(new FTaskRequestResponse((ETaskRequestResponseType)ResponseType), NativeMessage, false);
									}
									break;

									case ETaskRequestResponseType.SPECIFICATION:
									{
										AgentTaskSpecification TaskSpecificationMessage = (AgentTaskSpecification)TaskRequestResponseMessage;
										FGuid TaskGuid = new FGuid(TaskSpecificationMessage.TaskGuid.A,
																   TaskSpecificationMessage.TaskGuid.B,
																   TaskSpecificationMessage.TaskGuid.C,
																   TaskSpecificationMessage.TaskGuid.D);
										EJobTaskFlags TaskFlags = (EJobTaskFlags)TaskSpecificationMessage.TaskFlags;

										Char[] RawParametersData = TaskSpecificationMessage.Parameters.ToCharArray();
										if (RawParametersData.Length > 0)
										{
											// Pin the string data for the message
											PinnedStringData = new String(RawParametersData);
										}

										FTaskSpecificationMarshalHelper MarshalHelper = new FTaskSpecificationMarshalHelper();
										MarshalHelper.TaskGuid = TaskGuid;
										MarshalHelper.Parameters = FStringMarshaler.MarshalManagedToNative(PinnedStringData);
										MarshalHelper.Flags = TaskFlags;
										MarshalHelper.Cost = 0;
										MarshalHelper.DependencyCount = 0;
										MarshalHelper.Dependencies = IntPtr.Zero;

										NativeMessage = Marshal.AllocHGlobal(Marshal.SizeOf(typeof(FTaskSpecificationMarshalHelper)));
										Marshal.StructureToPtr(MarshalHelper, NativeMessage, false);
									}
									break;
								}
							}
							break;

							case EMessageType.TASK_STATE:
							{
								AgentTaskState TaskStateMessage = (AgentTaskState)ManagedMessage;

								// TODO: Assert that we have a valid Job GUID, since this must be the Instigator
								// TODO: Assert that the Job GUID of the message matches the ConnectionConfiguration.AgentJobGuid

								FGuid TaskGuid = new FGuid(TaskStateMessage.TaskGuid.A,
														   TaskStateMessage.TaskGuid.B,
														   TaskStateMessage.TaskGuid.C,
														   TaskStateMessage.TaskGuid.D);

								FTaskState NativeTaskStateMessage = new FTaskState(TaskGuid, (EJobTaskState)TaskStateMessage.TaskState);

								// If there is a message, be sure to pin and pass it
								if (TaskStateMessage.TaskMessage != null)
								{
									Char[] RawTaskMessageData = TaskStateMessage.TaskMessage.ToCharArray();
									if (RawTaskMessageData.Length > 0)
									{
										// Pin the string data for the message
										PinnedStringData = new String(RawTaskMessageData);
										NativeTaskStateMessage.TaskMessage = FStringMarshaler.MarshalManagedToNative(PinnedStringData);
									}
								}

								NativeMessage = Marshal.AllocHGlobal(Marshal.SizeOf(typeof(FTaskState)));
								Marshal.StructureToPtr(NativeTaskStateMessage, NativeMessage, false);
							}
							break;

							case EMessageType.INFO:
							{
								// Create the managed version of the info message
								AgentInfoMessage ManagedInfoMessage = (AgentInfoMessage)ManagedMessage;
								FInfoMessage NativeInfoMessage = new FInfoMessage("");
								if (ManagedInfoMessage.TextMessage != null)
								{
									Char[] RawTaskMessageData = ManagedInfoMessage.TextMessage.ToCharArray();
									if (RawTaskMessageData.Length > 0)
									{
										// Pin the string data for the message
										PinnedStringData = new String(RawTaskMessageData);
										NativeInfoMessage.TextMessage = FStringMarshaler.MarshalManagedToNative(PinnedStringData);
									}
								}

								NativeMessage = Marshal.AllocHGlobal(Marshal.SizeOf(typeof(FInfoMessage)));
								Marshal.StructureToPtr(NativeInfoMessage, NativeMessage, false);
							}
							break;

							case EMessageType.ALERT:
							{
								// Create the managed version of the info message
								AgentAlertMessage ManagedAlertMessage = (AgentAlertMessage)ManagedMessage;
								FGuid JobGuid = new FGuid(ManagedAlertMessage.JobGuid.A,
														  ManagedAlertMessage.JobGuid.B,
														  ManagedAlertMessage.JobGuid.C,
														  ManagedAlertMessage.JobGuid.D);
								FGuid ObjectGuid = new FGuid(ManagedAlertMessage.ObjectGuid.A,
															 ManagedAlertMessage.ObjectGuid.B,
															 ManagedAlertMessage.ObjectGuid.C,
															 ManagedAlertMessage.ObjectGuid.D);
								FAlertMessage NativeAlertMessage = new FAlertMessage(
									JobGuid, 
									ManagedAlertMessage.AlertLevel, 
									ObjectGuid, 
									ManagedAlertMessage.TypeId);
								if (ManagedAlertMessage.TextMessage != null)
								{
									Char[] RawTaskMessageData = ManagedAlertMessage.TextMessage.ToCharArray();
									if (RawTaskMessageData.Length > 0)
									{
										// Pin the string data for the message
										PinnedStringData = new String(RawTaskMessageData);
										NativeAlertMessage.TextMessage = FStringMarshaler.MarshalManagedToNative(PinnedStringData);
									}
								}
								NativeMessage = Marshal.AllocHGlobal(Marshal.SizeOf(typeof(FAlertMessage)));
								Marshal.StructureToPtr(NativeAlertMessage, NativeMessage, false);
							}
							break;

							default:
								// By default, just pass the message version and type through, but
								// any additional payload of a specialized type will be lost
								NativeMessage = Marshal.AllocHGlobal(Marshal.SizeOf(typeof(FMessage)));
								Marshal.StructureToPtr(new FMessage(ESwarmVersionValue.VER_1_0, (EMessageType)ManagedMessage.Type), NativeMessage, false);
							break;
						}

						// If a message was created to pass on to the user's callback, send it on
						if (NativeMessage != IntPtr.Zero)
						{
							// Call the user's callback function
							ThreadData.ConnectionCallback(NativeMessage, ThreadData.ConnectionCallbackData);

							// All finished with the message, free and set to null
							NativeMessage = IntPtr.Zero;
						}

						if (ManagedMessage.Type == EMessageType.QUIT)
						{
							// Return from this function, which will exit this message processing thread
							DebugLog.Write("Message queue thread shutting down from a QUIT message");
							return;
						}
					}

					// Reset the message handle
					ManagedMessage = null;
				}
			}
			catch (ThreadAbortException)
			{
				// An expected exception when closing the connection
				DebugLog.Write("Message queue thread shutting down normally after being closed by CloseConnection");
			}
			catch (Exception Ex)
			{
				DebugLog.Write("Error: Exception in the message queue thread: " + Ex.Message);

				// If the connection has thrown us an exception, close the connection
				DebugLog.Write("MessageThreadProc calling CleanupClosedConnection");
				ThreadData.Owner.CleanupClosedConnection();
			}

			// Only write out in debug
			DebugLog.Write("Message queue thread shutting down normally");
		}

		/**
		 * The function for the agent monitoring thread used to watch
		 * for potential crashed or hung agents
		 */
		static void MonitorThreadProc(Object ThreadParameters)
		{
			MessageThreadData ThreadData = (MessageThreadData)ThreadParameters;
			AgentConfiguration ConnectionConfiguration = ThreadData.ConnectionConfiguration;
			IAgentInterfaceWrapper Connection = ThreadData.Connection;

			bool NeedToCleanupClosedConnection = false;
			try
			{
				if (Connection != null)
				{
					// We'll just monitor the process itself to see if it exits
					Int32 ConnectionProcessID = ConnectionConfiguration.AgentProcessID;

					Process AgentProcess = Process.GetProcessById(ConnectionProcessID);
					if (AgentProcess != null)
					{
						// As long as the process is still running, yield
						bool KeepRunning = true;
						while (KeepRunning && !KillMonitorThread)
						{
							// Sleep for a little bit at the beginning of each iteration
							Thread.Sleep(1000);

							if (AgentProcess.HasExited == true)
							{
								// If the Agent process has exited, game over
								NeedToCleanupClosedConnection = true;
								KeepRunning = false;
							}
						}
						KillMonitorThread = false;
					}
				}
			}
			catch (ThreadAbortException)
			{
				// An expected exception when closing the connection
				DebugLog.Write("Agent monitor thread shutting down normally after being closed by CloseConnection");
			}
			catch (Exception Ex)
			{
				DebugLog.Write("Exception in the agent monitor thread: " + Ex.Message);

				// If the connection has thrown us an exception, close the connection
				NeedToCleanupClosedConnection = true;
			}

			// If needed, clean up the connection
			if (NeedToCleanupClosedConnection)
			{
				DebugLog.Write("MonitorThreadProc calling CleanupClosedConnection");
				ThreadData.Owner.CleanupClosedConnection();
			}

			DebugLog.Write("Agent monitor thread shutting down normally");
		}

		/*
		 * Start a timer going
		 */
		void StartTiming(String Name, bool Accum, Int64 Adder)
		{
			if (PerfTimerInstance != null)
			{
				PerfTimerInstance.Start(Name, Accum, Adder);
			}
		}

		void StartTiming(String Name, bool Accum)
		{
			StartTiming(Name, Accum, 0);
		}

		/*
		 * Stop a timer
		 */
		void StopTiming()
		{
			if (PerfTimerInstance != null)
			{
				PerfTimerInstance.Stop();
			}
		}

		/*
		 * Print out all the timing info
		 */
		void DumpTimings()
		{
			if (PerfTimerInstance != null)
			{
				String PerfMessage = PerfTimerInstance.DumpTimings();
				IntPtr PerfMessagePtr = Marshal.AllocHGlobal(Marshal.SizeOf(typeof(FInfoMessage)));
				Marshal.StructureToPtr(new FInfoMessage(PerfMessage), PerfMessagePtr, true);
				SendMessage(PerfMessagePtr);
				PerfTimerInstance = null;
			}
		}

		/*
		 * Clean up all of the remainders from a closed connection, including the network channels, etc.
		 */
		Int32 CleanupClosedConnection()
		{
			Monitor.Enter(CleanupClosedConnectionLock);

			// NOTE: Do not make any calls to the real Connection in here!
			// If, for any reason, the connection has died, calling into it from here will
			// end up causing this thread to hang, waiting for the dead connection to respond.
			DebugLog.Write("[Interface:CleanupClosedConnection] Closing all connections to the Agent");

			// Reset all necessary assigned variables
			AgentProcess = null;
			AgentProcessOwner = false;

			if (Connection != null)
			{
				// Notify the connection wrapper that the connection is gone
				Connection.SignalConnectionDropped();
				Connection = null;
			}
			ConnectionHandle = Constants.INVALID;
			ConnectionConfiguration = null;

			// Clean up and close up the connection callback
			if (ConnectionCallback != null)
			{
				IntPtr QuitMessage = Marshal.AllocHGlobal(Marshal.SizeOf(typeof(FMessage)));
				Marshal.StructureToPtr(new FMessage(ESwarmVersionValue.VER_1_0, EMessageType.QUIT), QuitMessage, false);
				ConnectionCallback(QuitMessage, ConnectionCallbackData);
			}
			ConnectionCallback = null;
			ConnectionCallbackData = IntPtr.Zero;

			// Close all associated channels
			if (OpenChannels.Count > 0)
			{
				foreach (ChannelInfo NextChannel in OpenChannels.Values)
				{
					// Just close the handle and we'll clear the entire list later
					if (NextChannel.ChannelFileStream != null)
					{
						NextChannel.ChannelFileStream.Close();
						NextChannel.ChannelFileStream = null;
					}
				}
				OpenChannels.Clear();
			}

			// Unregister the primary network communication channel
			if (NetworkChannel != null)
			{
				ChannelServices.UnregisterChannel(NetworkChannel);
				NetworkChannel = null;
			}

			Monitor.Exit(CleanupClosedConnectionLock);

			return Constants.SUCCESS;
		}

		/** All agent/swarm-specific variables */
		Process										AgentProcess;
		bool										AgentProcessOwner;
	
		/** All connection-specific variables */
		IAgentInterfaceWrapper						Connection;
		Int32										ConnectionHandle;
		Thread										ConnectionMessageThread;
		Thread										ConnectionMonitorThread;
		FConnectionCallback							ConnectionCallback;
		IntPtr										ConnectionCallbackData;
		ELogFlags									ConnectionLoggingFlags;
		AgentConfiguration							ConnectionConfiguration;
		Object										CleanupClosedConnectionLock;

		/** A convenience class for packing a few managed types together */
		class ChannelInfo
		{
			public String			ChannelName;
			public EChannelFlags	ChannelFlags;
			public Int32			ChannelHandle;
			public Stream			ChannelFileStream;

			// For buffered reads and writes
			public byte[]			ChannelData;
			public Int32			ChannelOffset;
		};

		ReaderWriterDictionary<Int32, ChannelInfo>	OpenChannels;
		Stack<byte[]>								FreeChannelWriteBuffers;
		Int32										BaseChannelHandle;

		/** While a job is being specified, collect the task specifications until it's done, then submit all at once */
		List<AgentTaskSpecification>				PendingTasks;

		/** A pair of locations needed for using channels */
		String										AgentCacheFolder;

		/** The network channel used to communicate with the Agent */
		TcpClientChannel							NetworkChannel;

		/** A helper class for timing info */
		PerfTimer									PerfTimerInstance;

		Int32 TryOpenConnection(FConnectionCallback CallbackFunc, IntPtr CallbackData, ELogFlags LoggingFlags)
		{
			try
			{
				// Allocate our new connection wrapper object
				Connection = new IAgentInterfaceWrapper();

				// Make sure the agent is alive and responsive before continuing
				EditorLog(EVerbosityLevel.Informative, "[TryOpenConnection] Testing the Agent");
				Hashtable InParameters = null;
				Hashtable OutParameters = null;
				bool AgentIsReady = false;
				while (!AgentIsReady)
				{
					try
					{
						// Simply try to call the method and if it doesn't throw
						// an exception, consider it a success
						Connection.Method(0, InParameters, ref OutParameters);
						AgentIsReady = true;
					}
					catch (Exception ex)
					{
						// Wait a little longer
						EditorLog(EVerbosityLevel.Critical, "[TryOpenConnection] Waiting for the agent to start up ...");
						EditorLog(EVerbosityLevel.Critical, ex.ToString());
						Thread.Sleep(5000);
					}
				}

				// Request an official connection to the Agent
				EditorLog(EVerbosityLevel.Informative, "[TryOpenConnection] Opening Connection to Agent");
				EditorLog(EVerbosityLevel.Informative, "[TryOpenConnection] Local Process ID is " + Process.GetCurrentProcess().Id.ToString());

				StartTiming("OpenConnection-Remote", false);
				ConnectionHandle = Connection.OpenConnection(AgentProcess, AgentProcessOwner, Process.GetCurrentProcess().Id, LoggingFlags, out ConnectionConfiguration);
				StopTiming();

				if (ConnectionHandle >= 0)
				{
					Log(EVerbosityLevel.Informative, ELogColour.Green, "[Interface:TryOpenConnection] Local connection established");

					// Spawn a thread to monitor the message queue
					MessageThreadData ThreadData = new MessageThreadData();
					ThreadData.Owner = this;
					ThreadData.Connection = Connection;
					ThreadData.ConnectionHandle = ConnectionHandle;
					ThreadData.ConnectionCallback = CallbackFunc;
					ThreadData.ConnectionCallbackData = CallbackData;
					ThreadData.ConnectionConfiguration = ConnectionConfiguration;

					// Launch the message queue thread
					ConnectionMessageThread = new Thread(new ParameterizedThreadStart(MessageThreadProc));
					ConnectionMessageThread.Name = "ConnectionMessageThread";
					ConnectionMessageThread.Start( ThreadData );

					// Launch the agent monitor thread
					ConnectionMonitorThread = new Thread(new ParameterizedThreadStart(MonitorThreadProc));
					ConnectionMonitorThread.Name = "ConnectionMonitorThread";
					ConnectionMonitorThread.Start(ThreadData);

					// Save the user's callback routine
					ConnectionCallback = CallbackFunc;
					ConnectionCallbackData = CallbackData;
					ConnectionLoggingFlags = LoggingFlags;
				}
			}
			catch (Exception Ex)
			{
				EditorLog(EVerbosityLevel.Critical, "[TryOpenConnection] Error: " + Ex.Message);
				EditorLog(EVerbosityLevel.Critical, Ex.ToString());
				ConnectionHandle = Constants.INVALID;
				Connection = null;
			}

			return ConnectionHandle;
		}

		void EnsureAgentIsRunning(string OptionsFolder)
		{
			// See if an agent is already running, and if not, launch one
			AgentProcess = null;
			AgentProcessOwner = false;

#if !__MonoCS__
			Process[] RunningSwarmAgents = Process.GetProcessesByName("SwarmAgent");
			if (RunningSwarmAgents.Length > 0)
			{
				// If any are running, just grab the first one
				AgentProcess = RunningSwarmAgents[0];
			}
#else
			Process[] RunningSwarmAgents = Process.GetProcessesByName("mono");
            if (RunningSwarmAgents.Length == 0)
            {
                RunningSwarmAgents = Process.GetProcessesByName("mono-sgen");
            }
			foreach (Process Iter in RunningSwarmAgents)
			{
				foreach(ProcessModule m in Iter.Modules)
				{
					if(m.ModuleName.Contains("SwarmAgent"))
					{
						AgentProcess = Iter;
						break;
					}
				}
				if(AgentProcess != null)
					break;
			}
			if (RunningSwarmAgents.Length > 0)
			{
				// If any are running, just grab the first one
				AgentProcess = RunningSwarmAgents[0];
			}
#endif

			if (AgentProcess == null)
			{
				String StartupPath = Path.GetDirectoryName(Assembly.GetExecutingAssembly().Location);
#if !__MonoCS__
				StartupPath += "/../DotNET/SwarmAgent.exe";
#else
				StartupPath += "/../Mac/SwarmAgent.exe";
#endif

				// If none, launch the Agent binary
				DebugLog.Write("[OpenConnection] Spawning agent: " + StartupPath);
				if (File.Exists(StartupPath))
				{
					String Arguments = "IAmSwarmAgent";
					if ( OptionsFolder.Length > 0 )
					{
						Arguments += " -OptionsFolder=\"" + OptionsFolder + "\"";
					}
					AgentProcess = Process.Start(StartupPath, Arguments);
					if (AgentProcess == null)
					{
						DebugLog.Write("[OpenConnection] Failed to start the Agent executable: " + StartupPath);
					}
					else
					{
						// If we started the process, we own it and control its lifetime
						AgentProcessOwner = true;
					}
				}
				else
				{
					DebugLog.Write("[OpenConnection] Failed to find the Agent executable: " + StartupPath);
				}
			}
		}

		/**
		 * Converts an native job specification to a managed version.
		 *
		 * @param Specification	Native job specification (may be empty)
		 * @param bIs64bit		Whether the job specification is 64-bit or not
		 * @return				Newly created managed job specification, or null if the specification was empty
		 */
		AgentJobSpecification ConvertJobSpecification(ref FJobSpecification Specification, bool bIs64bit)
		{
			if (Specification.ExecutableName == null)
			{
				return null;
			}
			else
			{
				// Convert the parameters from native to managed
				String ExecutableName = Specification.ExecutableName;
				String Parameters = Specification.Parameters;
				EJobTaskFlags Flags = (EJobTaskFlags)((Specification.Flags & ~EJobTaskFlags.FLAG_64BIT) | (bIs64bit ? EJobTaskFlags.FLAG_64BIT : 0));

				List<String> RequiredDependencies = null;
				if (Specification.RequiredDependencyCount > 0)
				{
					RequiredDependencies = new List<String>();
					for (UInt32 i = 0; i < Specification.RequiredDependencyCount; i++)
					{
						RequiredDependencies.Add(Specification.RequiredDependencies[i]);
					}
				}
				List<String> OptionalDependencies = null;
				if (Specification.OptionalDependencyCount > 0)
				{
					OptionalDependencies = new List<String>();
					for (UInt32 i = 0; i < Specification.OptionalDependencyCount; i++)
					{
						OptionalDependencies.Add(Specification.OptionalDependencies[i]);
					}
				}

				return new AgentJobSpecification(ConnectionConfiguration.AgentJobGuid, (EJobTaskFlags)Flags, ExecutableName, Parameters, RequiredDependencies, OptionalDependencies);
			}
		}

		/*
		 * Generates the full channel name, including Job path and staging area path if necessary
		 */
		String GenerateFullChannelName(String ManagedChannelName, EChannelFlags ChannelFlags)
		{
			if ((ChannelFlags & EChannelFlags.TYPE_PERSISTENT) != 0)
			{
				if ((ChannelFlags & EChannelFlags.ACCESS_WRITE) != 0)
				{
					// A persistent cache channel open for writing opens in the staging area
					String StagingAreaName = Path.Combine(AgentCacheFolder, "AgentStagingArea");
					return Path.Combine(StagingAreaName, ManagedChannelName);
				}
				else if ((ChannelFlags & EChannelFlags.ACCESS_READ) != 0)
				{
					// A persistent cache channel open for reading opens directly in the cache
					return Path.Combine(AgentCacheFolder, ManagedChannelName);
				}
			}
			else if ((ChannelFlags & EChannelFlags.TYPE_JOB_ONLY) != 0)
			{
				AgentGuid JobGuid = ConnectionConfiguration.AgentJobGuid;
				if (JobGuid == null)
				{
					// If there's no Job associated with this connection at this point, provide
					// a default GUID for one for debugging access to the agent cache
					JobGuid = new AgentGuid(0x00000123, 0x00004567, 0x000089ab, 0x0000cdef);
				}

				// A Job Channel opens directly in the Job-specific directory
				String JobsFolder = Path.Combine(AgentCacheFolder, "Jobs");
				String JobFolderName = Path.Combine(JobsFolder, "Job-" + JobGuid.ToString());
				return Path.Combine(JobFolderName, ManagedChannelName);
			}

			return "";
		}

		/*
		 * CacheFileAndConvertName
		 *
		 * Checks to see if the file with the correct timestamp and size exists in the cache
		 * If it does not, add it to the cache
		 */
		Int32 CacheFileAndConvertName(String FullPathName, out String CachedFileName, bool bUse64bitNamingConvention)
		{
			Int32 ReturnValue = Constants.INVALID;
			CachedFileName = "";

			// First, check that the original file exists
			if (File.Exists(FullPathName))
			{
				// The full cache name is based on the file name, the last modified timestamp on the file, and the file size
				FileInfo FullPathFileInfo = new FileInfo(FullPathName);

				// The last modified timestamp
				String LastModifiedTimeUTCString = FullPathFileInfo.LastWriteTimeUtc.ToString("yyyy-MM-dd_HH-mm-ss");

				// The file size, which we get by opening the file
				String FileSizeInBytesString = FullPathFileInfo.Length.ToString();

				String Suffix64bit = bUse64bitNamingConvention ? "-64bit" : "";

				// Compose the full cache name
				CachedFileName = String.Format("{0}_{1}_{2}{3}{4}",
					Path.GetFileNameWithoutExtension(FullPathName),
					LastModifiedTimeUTCString,
					FileSizeInBytesString,
					Suffix64bit,
					FullPathFileInfo.Extension);

				// Test the cache with the cached file name
				if (Connection.TestChannel(ConnectionHandle, CachedFileName) < 0)
				{
					// If not already in the cache, attempt to add it now
					ReturnValue = Connection.AddChannel(ConnectionHandle, FullPathName, CachedFileName);
				}
				else
				{
					ReturnValue = Constants.SUCCESS;
				}
			}
			else
			{
				ReturnValue = Constants.ERROR_FILE_FOUND_NOT;
			}

			// Didn't find the file, return failure
			return ReturnValue;
		}

		/*
		 * CacheAllFiles
		 *
		 * Ensures all executables and dependencies are correctly placed in the cache
		 */
		Int32 CacheAllFiles(AgentJobSpecification JobSpecification)
		{
			Int32 ReturnValue = Constants.INVALID;
			try
			{
				// Allocate the dictionary we'll use for the name mapping
				Dictionary<String, String> DependenciesOriginalNames = new Dictionary<String, String>();

				bool bUse64bitNamingConvention = (JobSpecification.JobFlags & EJobTaskFlags.FLAG_64BIT) != 0;

				// Check for and possibly cache the executable
				String OriginalExecutableFullPath = Path.GetFullPath(JobSpecification.ExecutableName);
				String CachedExecutableName;
				ReturnValue = CacheFileAndConvertName(OriginalExecutableFullPath, out CachedExecutableName, bUse64bitNamingConvention);
				if (ReturnValue < 0)
				{
					// Failed to cache the executable, return failure
					Log(EVerbosityLevel.Critical, ELogColour.Red, "[Interface:CacheAllFiles] Failed to cache the executable: " + OriginalExecutableFullPath);
					return ReturnValue;
				}

				JobSpecification.ExecutableName = CachedExecutableName;

				String OriginalExecutableName = Path.GetFileName(OriginalExecutableFullPath);
				DependenciesOriginalNames.Add(CachedExecutableName, OriginalExecutableName);

				// Check and cache all the required dependencies
				if ((JobSpecification.RequiredDependencies != null) &&
					(JobSpecification.RequiredDependencies.Count > 0))
				{
					for (Int32 i = 0; i < JobSpecification.RequiredDependencies.Count; i++)
					{
						String OriginalDependencyFullPath = Path.GetFullPath(JobSpecification.RequiredDependencies[i]);
						String CachedDependencyName;
						ReturnValue = CacheFileAndConvertName(OriginalDependencyFullPath, out CachedDependencyName, bUse64bitNamingConvention);
						if (ReturnValue < 0)
						{
							// Failed to cache the dependency, return failure
							Log(EVerbosityLevel.Critical, ELogColour.Red, "[Interface:CacheAllFiles] Failed to cache the required dependency: " + OriginalDependencyFullPath);
							return ReturnValue;
						}

						JobSpecification.RequiredDependencies[i] = CachedDependencyName;

						String OriginalDependencyName = Path.GetFileName(OriginalDependencyFullPath);
						DependenciesOriginalNames.Add(CachedDependencyName, OriginalDependencyName);
					}
				}
				// Check and cache any optional dependencies
				if ((JobSpecification.OptionalDependencies != null) &&
					(JobSpecification.OptionalDependencies.Count > 0))
				{
					for (Int32 i = 0; i < JobSpecification.OptionalDependencies.Count; i++)
					{
						String OriginalDependencyFullPath = JobSpecification.OptionalDependencies[i];
						String CachedDependencyName;
						ReturnValue = CacheFileAndConvertName(OriginalDependencyFullPath, out CachedDependencyName, bUse64bitNamingConvention);
						if (ReturnValue < 0)
						{
							// Failed to cache the dependency, log a warning
							Log(EVerbosityLevel.Verbose, ELogColour.Orange, "[Interface:CacheAllFiles] Failed to cache an optional dependency: " + OriginalDependencyFullPath);
						}
						else
						{
							JobSpecification.OptionalDependencies[i] = CachedDependencyName;

							String OriginalDependencyName = Path.GetFileName(OriginalDependencyFullPath);
							DependenciesOriginalNames.Add(CachedDependencyName, OriginalDependencyName);
						}
					}
				}

				// Set the newly created name mapping dictionary
				JobSpecification.DependenciesOriginalNames = DependenciesOriginalNames;

				ReturnValue = Constants.SUCCESS;
			}
			catch (Exception Ex)
			{
				Log(EVerbosityLevel.Critical, ELogColour.Red, "[Interface:CacheAllFiles] Error: " + Ex.Message);
				ReturnValue = Constants.ERROR_CONNECTION_DISCONNECTED;
				CleanupClosedConnection();
			}
			return ReturnValue;
		}

		Int32 CacheAllFiles(AgentTaskSpecification TaskSpecification)
		{
			Int32 ReturnValue = Constants.INVALID;
			try
			{
				TaskSpecification.DependenciesOriginalNames = null;
				if ((TaskSpecification.Dependencies != null) &&
					(TaskSpecification.Dependencies.Count > 0))
				{
					// Allocate the dictionary we'll use for the name mapping
					Dictionary<String, String> DependenciesOriginalNames = new Dictionary<String, String>();

					// Check and cache all the dependencies
					for (Int32 i = 0; i < TaskSpecification.Dependencies.Count; i++)
					{
						String OriginalDependencyFullPath = TaskSpecification.Dependencies[i];
						String CachedDependencyName;
						ReturnValue = CacheFileAndConvertName(OriginalDependencyFullPath, out CachedDependencyName, false);
						if (ReturnValue < 0)
						{
							// Failed to cache the dependency, return failure
							Log(EVerbosityLevel.Critical, ELogColour.Red, "[Interface:CacheAllFiles] Failed to cache the dependency: " + OriginalDependencyFullPath);
							return ReturnValue;
						}

						TaskSpecification.Dependencies[i] = CachedDependencyName;

						String OriginalDependencyName = Path.GetFileName(OriginalDependencyFullPath);
						DependenciesOriginalNames.Add(CachedDependencyName, OriginalDependencyName);
					}

					// Set the newly created name mapping dictionary
					TaskSpecification.DependenciesOriginalNames = DependenciesOriginalNames;
				}
				ReturnValue = Constants.SUCCESS;
			}
			catch (Exception Ex)
			{
				Log(EVerbosityLevel.Critical, ELogColour.Red, "[Interface:CacheAllFiles] Error: " + Ex.Message);
				ReturnValue = Constants.ERROR_CONNECTION_DISCONNECTED;
				CleanupClosedConnection();
			}
			return ReturnValue;
		}

		/*
		 * Flushes any buffered writes to the channel
		 */
		Int32 FlushChannel(ChannelInfo ChannelToFlush)
		{
			Int32 ReturnValue = Constants.INVALID;
			if ((ChannelToFlush.ChannelFlags & EChannelFlags.ACCESS_WRITE) != 0 &&
				(ChannelToFlush.ChannelOffset > 0) &&
				(ChannelToFlush.ChannelData != null) &&
				(ChannelToFlush.ChannelFileStream != null))
			{
				try
				{
					ChannelToFlush.ChannelFileStream.Write(ChannelToFlush.ChannelData, 0, ChannelToFlush.ChannelOffset);
					ReturnValue = ChannelToFlush.ChannelOffset;
				}
				catch (Exception Ex)
				{
					Log(EVerbosityLevel.Critical, ELogColour.Red, "[Interface:FlushChannel] Error: " + Ex.Message);
					ReturnValue = Constants.ERROR_CHANNEL_IO_FAILED;
				}
				ChannelToFlush.ChannelOffset = 0;
			}
			return ReturnValue;
		}

		private void EditorLog(EVerbosityLevel Verbosity, string Message)
		{
			if(SwarmInterfaceLogCppProc != null)
			{
				SwarmInterfaceLogCppProc(Verbosity, FStringMarshaler.MarshalManagedToNative(Message));
			}
			else
			{
				DebugLog.Write(Message);
			}
		}

	}
}
