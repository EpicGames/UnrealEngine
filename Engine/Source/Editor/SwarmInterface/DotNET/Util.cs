// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.Runtime.InteropServices;
using System.Threading;

using AgentInterface;

namespace NSwarm
{
	///////////////////////////////////////////////////////////////////////////////

	/**
	 * A custom marshaler for strings. MarshalAs(UnmanagedType.LPTStr), cannot be used for strings
	 * as it always assumes UTF-16, while depending on platform, we can have UTF-16 or UTF-32.
	 */
	public class FStringMarshaler
	{
		public static IntPtr MarshalManagedToNative(String Str)
		{
			if (Str == null)
			{
				return IntPtr.Zero;
			}
			
			Byte[] Buffer = new Byte[(Str.Length + 1) * CHAR_SIZE];
#if __MonoCS__
			System.Text.Encoding.UTF32.GetBytes(Str, 0, Str.Length, Buffer, 0);
#else
			System.Text.Encoding.Unicode.GetBytes(Str, 0, Str.Length, Buffer, 0);
#endif
			IntPtr NativeString = Marshal.AllocHGlobal(Buffer.Length);
			Marshal.Copy(Buffer, 0, NativeString, Buffer.Length);
			return NativeString;
		}
		
		public static String MarshalNativeToManaged(IntPtr NativeString)
		{
			if (NativeString == IntPtr.Zero)
			{
				return null;
			}

			Int32 Length = 0;
#if __MonoCS__
			while (Marshal.ReadInt32(NativeString, Length * CHAR_SIZE) != 0)
#else
			while (Marshal.ReadInt16(NativeString, Length * CHAR_SIZE) != 0)
#endif
			{
				Length++;
			}
			
			Byte[] Buffer = new Byte[Length * CHAR_SIZE];
			Marshal.Copy(NativeString, Buffer, 0, Length * CHAR_SIZE);
#if __MonoCS__
			return System.Text.Encoding.UTF32.GetString(Buffer);
#else
			return System.Text.Encoding.Unicode.GetString(Buffer);
#endif
		}

#if __MonoCS__
		const Int32 CHAR_SIZE = 4;
#else
		const Int32 CHAR_SIZE = 2;
#endif
	}

	/**
	 * A simple utility struct to manage simple interactions with GUIDs
	 */
	[StructLayout(LayoutKind.Sequential)]
	public class FGuid
	{
		/**
		 * Default constructor, initializes to default values
		 */
		public FGuid()
		{
			A = 0;
			B = 0;
			C = 0;
			D = 0;
		}

		/**
		 * Constructor, initializes to specified values
		 */
		public FGuid(UInt32 InA, UInt32 InB, UInt32 InC, UInt32 InD)
		{
			A = InA;
			B = InB;
			C = InC;
			D = InD;
		}

		/**
		 * Sets the values of the GUID
		 */
		void Set(UInt32 InA, UInt32 InB, UInt32 InC, UInt32 InD)
		{
			A = InA;
			B = InB;
			C = InC;
			D = InD;
		}

		/**
		 * Simple comparison operator
		 */
		public bool Equals(FGuid Other)
		{
			// Needed for use in Dictionary collections
			return (A == Other.A) &&
			       (B == Other.B) &&
			       (C == Other.C) &&
			       (D == Other.D);
		}

		public UInt32 A;
		public UInt32 B;
		public UInt32 C;
		public UInt32 D;
	}

	///////////////////////////////////////////////////////////////////////////////

	/**
	 * A simple base class for messages. For each version of the messaging interface
	 * a newly derived type will inherit from this class. The base class is used to
	 * simply carry lightweight loads for messages, i.e. just the message type, which
	 * may be enough information in itself. For additional message data, subclass and
	 * add any additional data there.
	 */
	[StructLayout(LayoutKind.Sequential)]
	public class FMessage
	{
		/**
		 * Default constructor, initializes to default values
		 */
		public FMessage()
		{
			Version = ESwarmVersionValue.VER_1_0;
			Type = EMessageType.NONE;
		}

		/**
		 * Constructor, initializes to specified values
		 *
		 * @param NewType The type of the message, one of EMessageType
		 */
		public FMessage(EMessageType NewType)
		{
			Version = ESwarmVersionValue.VER_1_0;
			Type = NewType;
		}

		/**
		 * Constructor, initializes to specified values
		 *
		 * @param NewVersion The version of the message format; one of ESwarmVersionValue
		 * @param NewType The type of the message, one of EMessageType
		 */
		public FMessage(ESwarmVersionValue NewVersion, EMessageType NewType)
		{
			Version = NewVersion;
			Type = NewType;
		}

		/** The version of the message format; one of ESwarmVersionValue */
		public ESwarmVersionValue		Version;
		/** The type of the message, one of EMessageType */
		public EMessageType				Type;
	}

	///////////////////////////////////////////////////////////////////////////////

	/**
	 * Implementation of a generic info message, which just includes generic text.
	 */
	[StructLayout(LayoutKind.Sequential, CharSet = CharSet.Auto)]
	public class FInfoMessage : FMessage
	{
		public FInfoMessage()
			: base(ESwarmVersionValue.VER_1_0, EMessageType.INFO)
		{
		}

		/**
		 * Constructor, initializes to default and specified values
		 */
		public FInfoMessage(String InTextMessage)
			: base(ESwarmVersionValue.VER_1_0, EMessageType.INFO)
		{
			TextMessage = FStringMarshaler.MarshalManagedToNative(InTextMessage);
		}

		/** Generic text message for informational purposes */
		public IntPtr	TextMessage;
	}

	///////////////////////////////////////////////////////////////////////////////

	/**
	 * Implementation of a alert message, which includes:
	 *
	 *	- The alert type:
	 *		a) warning
	 *		b) error
	 *		c) critical error
	 *	- The Job GUID
	 *	- The GUID of the item causing the issue
	 *	- A 32-bit field intended to identify the type of the item
	 *	- A string giving the issue message (which will be localized on the Unreal side of things).
	 */
	[StructLayout(LayoutKind.Sequential, CharSet = CharSet.Auto)]
	public class FAlertMessage : FMessage
	{
		public FAlertMessage()
			: base(ESwarmVersionValue.VER_1_0, EMessageType.ALERT)
		{
		}

		/**
		 * Constructor, initializes to default and specified values
		 */
		public FAlertMessage(FGuid InJobGuid, EAlertLevel InAlertLevel, FGuid InObjectGuid, int InTypeId)
			: base(ESwarmVersionValue.VER_1_0, EMessageType.ALERT)
		{
			JobGuid = InJobGuid;
			AlertLevel = InAlertLevel;
			ObjectGuid = InObjectGuid;
			TypeId = InTypeId;
			TextMessage = IntPtr.Zero;
		}
		/**
		 * Constructor, initializes to default and specified values
		 */
		public FAlertMessage( FGuid InJobGuid, EAlertLevel InAlertLevel, FGuid InObjectGuid, int InTypeId, String InTextMessage )
			: base(ESwarmVersionValue.VER_1_0, EMessageType.ALERT)
		{
			JobGuid = InJobGuid;
			AlertLevel = InAlertLevel;
			ObjectGuid = InObjectGuid;
			TypeId = InTypeId;
			TextMessage = FStringMarshaler.MarshalManagedToNative(InTextMessage);
		}

		/** The Job Guid */
		public FGuid			JobGuid;
		/** The type of alert */
		public EAlertLevel		AlertLevel;
		/** The identifier for the object that is associated with the issue */
		public FGuid			ObjectGuid;
		/** App-specific identifier for the type of the object */
		public int				TypeId;
		/** Generic text message for informational purposes */
		public IntPtr			TextMessage;
	};

	///////////////////////////////////////////////////////////////////////////////

	/**
	 * Implementation of a generic info message, which just includes generic text.
	 */
	[StructLayout(LayoutKind.Sequential)]
	public class FTimingMessage : FMessage
	{
		public FTimingMessage()
			: base(ESwarmVersionValue.VER_1_0, EMessageType.TIMING)
		{
		}

		/**
		 * Constructor, initializes to default and specified values
		 */
		public FTimingMessage( EProgressionState NewState, int InThreadNum )
			: base(ESwarmVersionValue.VER_1_0, EMessageType.TIMING)
		{
			State = NewState;
			ThreadNum = InThreadNum;
		}

		/** State that the distributed job is transitioning to */
		public EProgressionState	State;
		/** The thread this state is referring to */
		public int					ThreadNum;
	};

	///////////////////////////////////////////////////////////////////////////////

	/**
	 * Implementation of a task request response message. All uses include the GUID
	 * of the Job the request referred to. Currently used for these message types:
	 * 
	 * TASK_RELEASE: Signifies that the requester is no longer required to process
	 * any more Tasks. The requester is free to consider this Job completed.
	 * 
	 * TASK_RESERVATION: Sent back only if the Job specified is still active but
	 * no additional Tasks are available at this time.
	 *
	 * TASK_SPECIFICATION: Details a Task that can be worked on
	 */
	[StructLayout(LayoutKind.Sequential)]
	public class FTaskRequestResponse : FMessage
	{
		public FTaskRequestResponse()
			: base(ESwarmVersionValue.VER_1_0, EMessageType.TASK_REQUEST_RESPONSE)
		{
		}

		/**
		 * Constructor, initializes to default and specified values
		 */
		public FTaskRequestResponse( ETaskRequestResponseType NewResponseType )
			: base(ESwarmVersionValue.VER_1_0, EMessageType.TASK_REQUEST_RESPONSE)
		{
			ResponseType = NewResponseType;
		}

		/** The type of response this message is. Subclasses add any additional data */
		ETaskRequestResponseType	ResponseType;
	};

	///////////////////////////////////////////////////////////////////////////////

	/**
	 * Encapsulates information about a Job specification passed into BeginJobSpecification
	 */
	public class FJobSpecification
	{
		/**
		 * Default constructor, initializes to an empty (invalid) job.
		 */
		public FJobSpecification()
		{
			ExecutableName = null;
			Parameters = null;
			Flags = EJobTaskFlags.TASK_FLAG_USE_DEFAULTS;
			RequiredDependencies = null;
			RequiredDependencyCount = 0;
			OptionalDependencies = null;
			OptionalDependencyCount = 0;
			DescriptionKeys = null;
			DescriptionValues = null;
			DescriptionCount = 0;
		}

		/**
		 * Constructor, initializes to default and specified values
		 */
		public FJobSpecification(String JobExecutableName, String JobParameters, EJobTaskFlags JobFlags)
		{
			ExecutableName = JobExecutableName;
			Parameters = JobParameters;
			Flags = JobFlags;
			RequiredDependencies = null;
			RequiredDependencyCount = 0;
			OptionalDependencies = null;
			OptionalDependencyCount = 0;
			DescriptionKeys = null;
			DescriptionValues = null;
			DescriptionCount = 0;
		}

		/**
		 * Used to add channel dependencies to a Job. When an Agent runs this Job,
		 * it will ensure that all dependencies are satisfied prior to launching
		 * the executable. Note that the Job executable is an implied dependency.
		 *
		 * @param NewRequiredDependencies The list of additional required dependent channel names
		 * @param NewRequiredDependencyCount The number of elements in the NewRequiredDependencies list
		 * @param NewOptionalDependencies The list of additional optional dependent channel names
		 * @param NewOptionalDependencyCount The number of elements in the NewOptionalDependencies list
		 */
		void AddDependencies(String[] NewRequiredDependencies, UInt32 NewRequiredDependencyCount, String[] NewOptionalDependencies, UInt32 NewOptionalDependencyCount)
		{
			RequiredDependencies = NewRequiredDependencies;
			RequiredDependencyCount = NewRequiredDependencyCount;
			OptionalDependencies = NewOptionalDependencies;
			OptionalDependencyCount = NewOptionalDependencyCount;
		}

		void AddDescription(String[] NewDescriptionKeys, String[] NewDescriptionValues, UInt32 NewDescriptionCount )
		{
			DescriptionKeys = NewDescriptionKeys;
			DescriptionValues = NewDescriptionValues;
			DescriptionCount = NewDescriptionCount;
		}

		/** The Job's executable name and parameter string */
		public String			ExecutableName;
		public String			Parameters;

		/** Flags used to control the behavior of the executing Job */
		public EJobTaskFlags	Flags;
		/** Any additional Job dependencies */
		public String[]			RequiredDependencies;
		public UInt32			RequiredDependencyCount;
		public String[]			OptionalDependencies;
		public UInt32			OptionalDependencyCount;

		/** Optional Job description values in key/value form */
		public String[]			DescriptionKeys;
		public String[]			DescriptionValues;
		public UInt32			DescriptionCount;
	};

	[StructLayout(LayoutKind.Sequential, CharSet = CharSet.Auto)]
	class FJobSpecificationMarshalHelper
	{
		public IntPtr			ExecutableName;
		public IntPtr			Parameters;
		public EJobTaskFlags	Flags;
		public IntPtr			RequiredDependencies;
		public UInt32			RequiredDependencyCount;
		public IntPtr			OptionalDependencies;
		public UInt32			OptionalDependencyCount;
		public IntPtr			DescriptionKeys;
		public IntPtr			DescriptionValues;
		public UInt32			DescriptionCount;
	}

	///////////////////////////////////////////////////////////////////////////////

	/**
	 * Encapsulates information about a Task specification passed into AddTask and
	 * later sent in response to a TASK_REQUEST message
	 */
	public class FTaskSpecification : FTaskRequestResponse
	{
		/**
		 * Constructor, initializes to default and specified values
		 */
		public FTaskSpecification(FGuid TaskTaskGuid, String TaskParameters, EJobTaskFlags TaskFlags)
			: base(ETaskRequestResponseType.SPECIFICATION)
		{
			TaskGuid = TaskTaskGuid;
			Parameters = TaskParameters;
			Flags = TaskFlags;
			Dependencies = null;
			DependencyCount = 0;
		}

		/**
		 * Used to add channel dependencies to a Task. When an Agent runs this Task,
		 * it will ensure that all dependencies are satisfied prior to giving the
		 * Task to the requester.
		 *
		 * @param NewDependencies The list of additional dependent channel names
		 * @param NewDependencyCount The number of elements in the NewDependencies list
		 */
		void AddDependencies(String[] NewDependencies, UInt32 NewDependencyCount )
		{
			Dependencies = NewDependencies;
			DependencyCount = NewDependencyCount;
		}

		/** The GUID used for identifying the Task being referred to */
		public FGuid			TaskGuid;

		/** The Task's parameter string specified with AddTask */
		public String			Parameters;

		/** Flags used to control the behavior of the Task, subject to overrides from the containing Job */
		public EJobTaskFlags	Flags;

		/** The Task's cost, relative to all other Tasks in the same Job, used for even distribution and scheduling */
		public UInt32			Cost;

		/** Any additional Task dependencies */
		public String[]			Dependencies;
		public UInt32			DependencyCount;
	}

	[StructLayout(LayoutKind.Sequential, CharSet = CharSet.Auto)]
	public class FTaskSpecificationMarshalHelper : FTaskRequestResponse
	{
		public FTaskSpecificationMarshalHelper()
			: base(ETaskRequestResponseType.SPECIFICATION)
		{
		}
		public FGuid			TaskGuid;
		public IntPtr			Parameters;
		public EJobTaskFlags	Flags;
		public UInt32			Cost;
		public IntPtr			Dependencies;
		public UInt32			DependencyCount;
	}

	///////////////////////////////////////////////////////////////////////////////

	/**
	 * Encapsulates information about a Job's state, used to communicate
	 * back to the Instigator
	 */
	[StructLayout(LayoutKind.Sequential, CharSet = CharSet.Auto)]
	public class FJobState : FMessage
	{
		public FJobState()
			: base(ESwarmVersionValue.VER_1_0, EMessageType.JOB_STATE)
		{
		}

		/**
		 * Constructor, initializes to specified values
		 */
		public FJobState(FGuid NewJobGuid, EJobTaskState NewJobState)
			: base(ESwarmVersionValue.VER_1_0, EMessageType.JOB_STATE)
		{
			JobGuid = NewJobGuid;
			JobState = NewJobState;
			JobMessage = IntPtr.Zero;
			JobExitCode = 0;
			JobRunningTime = 0.0;
		}

		/** The Job GUID used for identifying the Job */
		public FGuid			JobGuid;

		/** The current state and arbitrary message */
		public EJobTaskState	JobState;
		public IntPtr			JobMessage;

		/** Various stats, including run time, exit codes, etc. */
		public Int32			JobExitCode;
		public double			JobRunningTime;
	}

	///////////////////////////////////////////////////////////////////////////////

	/**
	 * Encapsulates information about a Task's state, used to communicate
	 * back to the Instigator
	 */
	[StructLayout(LayoutKind.Sequential, CharSet = CharSet.Auto)]
	public class FTaskState : FMessage
	{
		public FTaskState()
			: base(ESwarmVersionValue.VER_1_0, EMessageType.TASK_STATE)
		{
		}

		/**
		 * Constructor, initializes to specified values
		 */
		public FTaskState(FGuid NewTaskGuid, EJobTaskState NewTaskState)
			: base(ESwarmVersionValue.VER_1_0, EMessageType.TASK_STATE)
		{
			TaskGuid = NewTaskGuid;
			TaskState = NewTaskState;
			TaskMessage = IntPtr.Zero;
			TaskExitCode = 0;
			TaskRunningTime = 0.0;
		}

		/** The Task GUID used for identifying the Task */
		public FGuid			TaskGuid;

		/** The current Task state and arbitrary message */
		public EJobTaskState	TaskState;
		public IntPtr			TaskMessage;

		/** Various stats, including run time, exit codes, etc. */
		public Int32			TaskExitCode;
		public double			TaskRunningTime;
	}

	///////////////////////////////////////////////////////////////////////////////

	/**
	 * A thread-safe reader-writer-locked Dictionary wrapper
	 */
	public class ReaderWriterDictionary<TKey, TValue>
	{
		/**
		 * The key to protecting the contained dictionary properly
		 */
		ReaderWriterLock AccessLock;

		/**
		 * The protected dictionary
		 */
		Dictionary<TKey, TValue> ProtectedDictionary;

		public ReaderWriterDictionary()
		{
			AccessLock = new ReaderWriterLock();
			ProtectedDictionary = new Dictionary<TKey, TValue>();
		}

		/**
		 * Wrappers around each method we need to protect in the dictionary
		 */
		public void Add(TKey Key, TValue Value)
		{
			// Modifies the collection, use a writer lock
			AccessLock.AcquireWriterLock(Timeout.Infinite);
			try
			{
				ProtectedDictionary.Add(Key, Value);
			}
			finally
			{
				AccessLock.ReleaseWriterLock();
			}
		}

		public bool Remove(TKey Key)
		{
			// Modifies the collection, use a writer lock
			AccessLock.AcquireWriterLock(Timeout.Infinite);
			bool ReturnValue = false;
			try
			{
				ReturnValue = ProtectedDictionary.Remove(Key);
			}
			finally
			{
				AccessLock.ReleaseWriterLock();
			}
			return ReturnValue;
		}

		public void Clear()
		{
			// Modifies the collection, use a writer lock
			AccessLock.AcquireWriterLock(Timeout.Infinite);
			try
			{
				ProtectedDictionary.Clear();
			}
			finally
			{
				AccessLock.ReleaseWriterLock();
			}
		}

		public bool TryGetValue(TKey Key, out TValue Value)
		{
			// Does not modify the collection, use a reader lock
			AccessLock.AcquireReaderLock(Timeout.Infinite);
			bool ReturnValue = false;
			try
			{
				ReturnValue = ProtectedDictionary.TryGetValue(Key, out Value);
			}
			finally
			{
				AccessLock.ReleaseReaderLock();
			}
			return ReturnValue;
		}

		public Dictionary<TKey, TValue>.ValueCollection Values
		{
			get
			{
				AccessLock.AcquireReaderLock(Timeout.Infinite);
				Dictionary<TKey, TValue>.ValueCollection CopyOfValues = null;
				try
				{
					CopyOfValues = new Dictionary<TKey, TValue>.ValueCollection(ProtectedDictionary);
				}
				finally
				{
					AccessLock.ReleaseReaderLock();
				}
				return CopyOfValues;
			}
			private set
			{
				Values = value;
			}
		}

		public Int32 Count
		{
			get
			{
				// Does not modify the collection, use a reader lock
				AccessLock.AcquireReaderLock(Timeout.Infinite);
				Int32 ReturnValue = 0;
				try
				{
					ReturnValue = ProtectedDictionary.Count;
				}
				finally
				{
					AccessLock.ReleaseReaderLock();
				}
				return ReturnValue;
			}
		}
	}
}
