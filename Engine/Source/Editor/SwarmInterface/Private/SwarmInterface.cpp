// Copyright Epic Games, Inc. All Rights Reserved.

#include "SwarmInterface.h"

#if PLATFORM_WINDOWS && !PLATFORM_CPU_ARM_FAMILY

// To avoid compile problems with C++/CLI in VS11
#pragma warning(disable:4538)
#pragma warning(disable:4564)

#pragma warning(disable:4100) // unreferenced formal parameter

#include "Windows/AllowWindowsPlatformTypes.h"

// Define WIN32_LEAN_AND_MEAN to exclude rarely-used services from windows headers.
#define WIN32_LEAN_AND_MEAN
#include <windows.h>
#include <assert.h>
#include <metahost.h>

#include <ErrorRep.h>
#include <Werapi.h>
#include <DbgHelp.h>

#pragma comment(lib, "Faultrep.lib")
#pragma comment(lib, "wer.lib")

#pragma comment(lib, "mscoree.lib")

EXTERN_C IMAGE_DOS_HEADER __ImageBase;

#include "Windows/HideWindowsPlatformTypes.h"

#endif // PLATFORM_WINDOWS && !PLATFORM_CPU_ARM_FAMILY


namespace NSwarm
{

/**
 * The C++ implementation of FSwarmInterface
 */
class FSwarmInterfaceImpl : public FSwarmInterface
{
public:
	FSwarmInterfaceImpl( void );
	virtual ~FSwarmInterfaceImpl( void );
	virtual int32 OpenConnection( FConnectionCallback CallbackFunc, void* CallbackData, TLogFlags LoggingFlags, const TCHAR* OptionsFolder );
	virtual int32 CloseConnection( void );
	virtual int32 SendMessage( const FMessage& Message );
	virtual int32 AddChannel( const TCHAR* FullPath, const TCHAR* ChannelName );
	virtual int32 TestChannel( const TCHAR* ChannelName );
	virtual int32 OpenChannel( const TCHAR* ChannelName, TChannelFlags ChannelFlags );
	virtual int32 CloseChannel( int32 Channel );
	virtual int32 WriteChannel( int32 Channel, const void* Data, int32 DataSize );
	virtual int32 ReadChannel( int32 Channel, void* Data, int32 DataSize );
	virtual int32 OpenJob( const FGuid& JobGuid );
	virtual int32 BeginJobSpecification( const FJobSpecification& Specification32, const FJobSpecification& Specification64 );
	virtual int32 AddTask( const FTaskSpecification& Specification );
	virtual int32 EndJobSpecification( void );
	virtual int32 CloseJob( void );
	virtual int32 Log( TVerbosityLevel Verbosity, TLogColour TextColour, const TCHAR* Message );
	static bool InitSwarmInterfaceManaged(const TCHAR* SwarmInterfacePath);
};

#if !USE_LOCAL_SWARM_INTERFACE
/**
 * @return The Swarm singleton
 */
FSwarmInterface* FSwarmInterface::GInstance = NULL;

bool FSwarmInterface::Initialize(const TCHAR* SwarmInterfacePath)
{
	if (GInstance == NULL)
	{
		if (!FSwarmInterfaceImpl::InitSwarmInterfaceManaged(SwarmInterfacePath))
		{
			return false;
		}
		else
		{
			GInstance = new FSwarmInterfaceImpl();
		}
	}

	return true;
}

FSwarmInterface& FSwarmInterface::Get( void )
{
	return( *GInstance ); 
}
#endif
///////////////////////////////////////////////////////////////////////////////
///////////////////////////////////////////////////////////////////////////////

typedef int32 (*SwarmOpenConnectionProc)(FConnectionCallback CallbackFunc, void* CallbackData, TLogFlags LoggingFlags, const TCHAR* OptionsFolder);
typedef int32 (*SwarmCloseConnectionProc)(void);
typedef int32 (*SwarmSendMessageProc)(const FMessage* Message);
typedef int32 (*SwarmAddChannelProc)(const TCHAR* FullPath, const TCHAR* ChannelName);
typedef int32 (*SwarmTestChannelProc)(const TCHAR* ChannelName);
typedef int32 (*SwarmOpenChannelProc)(const TCHAR* ChannelName, TChannelFlags ChannelFlags);
typedef int32 (*SwarmCloseChannelProc)(int32 Channel);
typedef int32 (*SwarmWriteChannelProc)(int32 Channel, const void* Data, int32 DataSize);
typedef int32 (*SwarmReadChannelProc)(int32 Channel, void* Data, int32 DataSize);
typedef int32 (*SwarmOpenJobProc)(const FGuid* JobGuid);
typedef int32 (*SwarmBeginJobSpecificationProc)(const FJobSpecification* Specification32, const FJobSpecification* Specification64);
typedef int32 (*SwarmAddTaskProc)(const FTaskSpecification* Specification);
typedef int32 (*SwarmEndJobSpecificationProc)(void);
typedef int32 (*SwarmCloseJobProc)(void);
typedef int32 (*SwarmLogProc)(TVerbosityLevel Verbosity, TLogColour TextColour, const TCHAR* Message);

static SwarmOpenConnectionProc SwarmOpenConnection;
static SwarmCloseConnectionProc SwarmCloseConnection;
static SwarmSendMessageProc SwarmSendMessage;
static SwarmAddChannelProc SwarmAddChannel;
static SwarmTestChannelProc SwarmTestChannel;
static SwarmOpenChannelProc SwarmOpenChannel;
static SwarmCloseChannelProc SwarmCloseChannel;
static SwarmWriteChannelProc SwarmWriteChannel;
static SwarmReadChannelProc SwarmReadChannel;
static SwarmOpenJobProc SwarmOpenJob;
static SwarmBeginJobSpecificationProc SwarmBeginJobSpecification;
static SwarmAddTaskProc SwarmAddTask;
static SwarmEndJobSpecificationProc SwarmEndJobSpecification;
static SwarmCloseJobProc SwarmCloseJob;
static SwarmLogProc SwarmLog;

extern "C" DLLEXPORT void RegisterSwarmOpenConnectionProc(SwarmOpenConnectionProc Proc) { SwarmOpenConnection = Proc; }
extern "C" DLLEXPORT void RegisterSwarmCloseConnectionProc(SwarmCloseConnectionProc Proc) { SwarmCloseConnection = Proc; }
extern "C" DLLEXPORT void RegisterSwarmSendMessageProc(SwarmSendMessageProc Proc) { SwarmSendMessage = Proc; }
extern "C" DLLEXPORT void RegisterSwarmAddChannelProc(SwarmAddChannelProc Proc) { SwarmAddChannel = Proc; }
extern "C" DLLEXPORT void RegisterSwarmTestChannelProc(SwarmTestChannelProc Proc) { SwarmTestChannel = Proc; }
extern "C" DLLEXPORT void RegisterSwarmOpenChannelProc(SwarmOpenChannelProc Proc) { SwarmOpenChannel = Proc; }
extern "C" DLLEXPORT void RegisterSwarmCloseChannelProc(SwarmCloseChannelProc Proc) { SwarmCloseChannel = Proc; }
extern "C" DLLEXPORT void RegisterSwarmWriteChannelProc(SwarmWriteChannelProc Proc) { SwarmWriteChannel = Proc; }
extern "C" DLLEXPORT void RegisterSwarmReadChannelProc(SwarmReadChannelProc Proc) { SwarmReadChannel = Proc; }
extern "C" DLLEXPORT void RegisterSwarmOpenJobProc(SwarmOpenJobProc Proc) { SwarmOpenJob = Proc; }
extern "C" DLLEXPORT void RegisterSwarmBeginJobSpecificationProc(SwarmBeginJobSpecificationProc Proc) { SwarmBeginJobSpecification = Proc; }
extern "C" DLLEXPORT void RegisterSwarmAddTaskProc(SwarmAddTaskProc Proc) { SwarmAddTask = Proc; }
extern "C" DLLEXPORT void RegisterSwarmEndJobSpecificationProc(SwarmEndJobSpecificationProc Proc) { SwarmEndJobSpecification = Proc; }
extern "C" DLLEXPORT void RegisterSwarmCloseJobProc(SwarmCloseJobProc Proc) { SwarmCloseJob = Proc; }
extern "C" DLLEXPORT void RegisterSwarmLogProc(SwarmLogProc Proc) { SwarmLog = Proc; }

DECLARE_LOG_CATEGORY_EXTERN(LogSwarmInterface, Verbose, All);
DEFINE_LOG_CATEGORY(LogSwarmInterface)

extern "C" DLLEXPORT void SwarmInterfaceLog(TVerbosityLevel Verbosity, const TCHAR* Message)
{
	switch (Verbosity)
	{
	case VERBOSITY_Critical:
		UE_LOG_CLINKAGE(LogSwarmInterface, Error, TEXT("%s"), Message);
		break;
	case VERBOSITY_Complex:
		UE_LOG_CLINKAGE(LogSwarmInterface, Warning, TEXT("%s"), Message);
		break;
	default:
		UE_LOG_CLINKAGE(LogSwarmInterface, Log, TEXT("%s"), Message);
		break;
	}
}

FSwarmInterfaceImpl::FSwarmInterfaceImpl( void )
{
}

FSwarmInterfaceImpl::~FSwarmInterfaceImpl( void )
{
}

/**
 * Opens a new connection to the Swarm
 *
 * @param CallbackFunc The callback function Swarm will use to communicate back to the Instigator
 *
 * @return An INT containing the error code (if < 0) or the handle (>= 0) which is useful for debugging only
 */
int32 FSwarmInterfaceImpl::OpenConnection( FConnectionCallback CallbackFunc, void* CallbackData, TLogFlags LoggingFlags, const TCHAR* OptionsFolder )
{
	// CallbackFunc can be NULL
	// CallbackData can be NULL
	return SwarmOpenConnection(CallbackFunc, CallbackData, LoggingFlags, OptionsFolder);
}

/**
 * Closes an existing connection to the Swarm
 *
 * @return INT error code (< 0 is error)
 */
int32 FSwarmInterfaceImpl::CloseConnection( void )
{
	return SwarmCloseConnection();
}

/**
 * Sends a message to an Agent (return messages are sent via the FConnectionCallback)
 *
 * @param Message The message being sent
 *
 * @return INT error code (< 0 is error)
 */
int32 FSwarmInterfaceImpl::SendMessage( const FMessage& Message )
{
	return SwarmSendMessage(&Message);
}

/**
 * Adds an existing file to the cache. Note, any existing channel with the same
 * name will be overwritten.
 *
 * @param FullPath The full path name to the file that should be copied into the cache
 * @param ChannelName The name of the channel once it's in the cache
 *
 * @return INT error code (< 0 is error)
 */
int32 FSwarmInterfaceImpl::AddChannel( const TCHAR* FullPath, const TCHAR* ChannelName )
{
	if (FullPath == NULL)
	{
		return SWARM_ERROR_INVALID_ARG1;
	}

	if (ChannelName == NULL)
	{
		return SWARM_ERROR_INVALID_ARG2;
	}

	int32 ReturnValue = SwarmAddChannel(FullPath, ChannelName);
	if (ReturnValue < 0)
	{
		SendMessage(FInfoMessage(TEXT("Error, fatal in AddChannel")));
	}

	return ReturnValue;
}

/**
 * Determines if the named channel is in the cache
 *
 * @param ChannelName The name of the channel to look for
 *
 * @return INT error code (< 0 is error)
 */
int32 FSwarmInterfaceImpl::TestChannel( const TCHAR* ChannelName )
{
	if( ChannelName == NULL )
	{
		return SWARM_ERROR_INVALID_ARG1;
	}

	int32 ReturnValue = SwarmTestChannel( ChannelName );
	// Check for the one, known error code (file not found)
	if( ( ReturnValue < 0 ) &&
		( ReturnValue != SWARM_ERROR_FILE_FOUND_NOT ) )
	{
		SendMessage( FInfoMessage( TEXT( "Error, fatal in TestChannel" ) ) );
	}

	return( ReturnValue );
}

/**
 * Opens a data channel for streaming data into the cache associated with an Agent
 *
 * @param ChannelName The name of the channel being opened
 * @param ChannelFlags The mode, access, and other attributes of the channel being opened
 *
 * @return A handle to the opened channel (< 0 is error)
 */
int32 FSwarmInterfaceImpl::OpenChannel( const TCHAR* ChannelName, TChannelFlags ChannelFlags )
{
	if( ChannelName == NULL )
	{
		return( SWARM_ERROR_INVALID_ARG1 );
	}

	int32 ReturnValue = SwarmOpenChannel( ChannelName, ChannelFlags );
	if( ReturnValue < 0 && (ChannelFlags & SWARM_CHANNEL_ACCESS_WRITE) )
	{
		SendMessage( FInfoMessage( TEXT( "Error, fatal in OpenChannel" ) ) );
	}

	return( ReturnValue );
}

/**
 * Closes an open channel
 *
 * @param Channel An open channel handle, returned by OpenChannel
 *
 * @return INT error code (< 0 is error)
 */
int32 FSwarmInterfaceImpl::CloseChannel( int32 Channel )
{
	if( Channel < 0 )
	{
		return( SWARM_ERROR_INVALID_ARG1 );
	}

	int32 ReturnValue = SwarmCloseChannel( Channel );
	if( ReturnValue < 0 )
	{
		SendMessage( FInfoMessage( TEXT( "Error, fatal in CloseChannel" ) ) );
	}

	return( ReturnValue );
}

/**
 * Writes the provided data to the open channel opened for WRITE
 *
 * @param Channel An open channel handle, returned by OpenChannel
 * @param Data Source buffer for the write
 * @param Data Size of the source buffer
 *
 * @return The number of bytes written (< 0 is error)
 */
int32 FSwarmInterfaceImpl::WriteChannel( int32 Channel, const void* Data, int32 DataSize )
{
	if( Channel < 0 )
	{
		return( SWARM_ERROR_INVALID_ARG1 );
	}

	if( Data == NULL )
	{
		return( SWARM_ERROR_INVALID_ARG2 );
	}

	if( DataSize < 0 )
	{
		return( SWARM_ERROR_INVALID_ARG3 );
	}

	int32 ReturnValue = SwarmWriteChannel( Channel, Data, DataSize );
	if( ReturnValue < 0 )
	{
		SendMessage( FInfoMessage( TEXT( "Error, fatal in WriteChannel" ) ) );
	}

	return( ReturnValue );
}

/**
 * Reads data from a channel opened for READ into the provided buffer
 *
 * @param Channel An open channel handle, returned by OpenChannel
 * @param Data Destination buffer for the read
 * @param Data Size of the destination buffer
 *
 * @return The number of bytes read (< 0 is error)
 */
int32 FSwarmInterfaceImpl::ReadChannel( int32 Channel, void* Data, int32 DataSize )
{
	if( Channel < 0 )
	{
		return( SWARM_ERROR_INVALID_ARG1 );
	}

	if( Data == NULL )
	{
		return( SWARM_ERROR_INVALID_ARG2 );
	}

	if( DataSize < 0 )
	{
		return( SWARM_ERROR_INVALID_ARG3 );
	}

	int32 ReturnValue = SwarmReadChannel( Channel, Data, DataSize );
	if( ReturnValue < 0 )
	{
		SendMessage( FInfoMessage( TEXT( "Error, fatal in ReadChannel" ) ) );
	}

	return( ReturnValue );
}

/**
 * Opens a Job session, which allows a Job to be specified, Tasks added, Job
 * channels opened and used, etc. When the Job is complete and no more Job
 * related data is needed from the Swarm, call CloseJob.
 *
 * @param JobGuid A GUID that uniquely identifies this Job, generated by the caller
 *
 * @return INT Error code (< 0 is an error)
 */
int32 FSwarmInterfaceImpl::OpenJob( const FGuid& JobGuid )
{
	int32 ReturnValue = SwarmOpenJob( &JobGuid );
	if( ReturnValue < 0 )
	{
		SendMessage( FInfoMessage( TEXT(" Error, fatal in OpenJob" ) ) );
	}

	return( ReturnValue );
}

/**
 * Begins a Job specification, which allows a series of Tasks to be specified
 * via AddTask. When Tasks are done being specified, call EndJobSpecification.
 *
 * The default behavior will be to execute the Job executable with the
 * specified parameters. If Tasks are added for the Job, they are expected
 * to be requested by the executable run for the Job. If no Tasks are added
 * for the Job, it is expected that the Job executable will perform its
 * operations without additional Task input from Swarm.
 *
 * @param Specification32 A structure describing a new 32-bit Job
 * @param Specification64 A structure describing a new 64-bit Job
 *
 * @return INT Error code (< 0 is an error)
 */
int32 FSwarmInterfaceImpl::BeginJobSpecification( const FJobSpecification& Specification32, const FJobSpecification& Specification64 )
{
	if( Specification32.ExecutableName == NULL && Specification64.ExecutableName == NULL )
	{
		return( SWARM_ERROR_INVALID_ARG );
	}

	if( Specification32.Parameters == NULL && Specification64.Parameters == NULL )
	{
		return( SWARM_ERROR_INVALID_ARG );
	}

	if( (Specification32.RequiredDependencyCount > 0 && Specification32.RequiredDependencies == NULL) ||
		(Specification32.OptionalDependencyCount > 0 && Specification32.OptionalDependencies == NULL) ||
		(Specification64.RequiredDependencyCount > 0 && Specification64.RequiredDependencies == NULL) ||
		(Specification64.OptionalDependencyCount > 0 && Specification64.OptionalDependencies == NULL) )
	{
		return( SWARM_ERROR_INVALID_ARG );
	}

	int32 ReturnValue = SwarmBeginJobSpecification( &Specification32, &Specification64 );
	if( ReturnValue < 0 )
	{
		SendMessage( FInfoMessage( TEXT("Error, fatal in BeginJobSpecification" ) ) );
	}

	return( ReturnValue );
}

/**
 * Adds a Task to the current Job
 *
 * @param Specification A structure describing the new Task
 *
 * @return INT Error code (< 0 is an error)
 */
int32 FSwarmInterfaceImpl::AddTask( const FTaskSpecification& Specification )
{
	if( Specification.Parameters == NULL )
	{
		return( SWARM_ERROR_INVALID_ARG );
	}

	if( ( Specification.DependencyCount > 0 ) &&
		( Specification.Dependencies == NULL ) )
	{
		return( SWARM_ERROR_INVALID_ARG );
	}

	int32 ReturnValue = SwarmAddTask( &Specification );
	if( ReturnValue < 0 )
	{
		SendMessage( FInfoMessage( TEXT( "Error, fatal in AddTask" ) ) );
	}

	return( ReturnValue );
}

/**
 * Ends the Job specification, after which no additional Tasks may be defined. Also,
 * this is generally the point when the Agent will validate and launch the Job executable,
 * potentially distributing the Job to other Agents.
 *
 * @return INT Error code (< 0 is an error)
 */
int32 FSwarmInterfaceImpl::EndJobSpecification( void )
{
	int32 ReturnValue = SwarmEndJobSpecification();
	if( ReturnValue < 0 )
	{
		SendMessage( FInfoMessage( TEXT("Error, fatal in EndJobSpecification" ) ) );
	}

	return( ReturnValue );
}

/**
 * Ends the definition period of a Job
 *
 * @param JobGuid The GUID of the Job specification
 *
 * @return INT error code (< 0 is error)
 */
int32 FSwarmInterfaceImpl::CloseJob( void )
{
	int32 ReturnValue = SwarmCloseJob();
	if( ReturnValue < 0 )
	{
		SendMessage( FInfoMessage( TEXT("Error, fatal in CloseJob" ) ) );
	}

	return( ReturnValue );
}

/**
 * Adds a line of text to the Agent log window
 *
 * @param Verbosity the importance of this message
 * @param TextColour the colour of the text
 * @param Message the line of text to add
 */
int32 FSwarmInterfaceImpl::Log( TVerbosityLevel Verbosity, TLogColour TextColour, const TCHAR* Message )
{
	if( Message == NULL )
	{
		return( SWARM_ERROR_NULL_POINTER );
	}

	int32 ReturnValue = SwarmLog( Verbosity, TextColour, Message );

	return( ReturnValue );
}

bool FSwarmInterfaceImpl::InitSwarmInterfaceManaged(const TCHAR* SwarmInterfaceDLLPath)
{
#if PLATFORM_WINDOWS && !PLATFORM_CPU_ARM_FAMILY
	ICLRMetaHost *MetaHost = NULL;
	ICLRRuntimeHost* RuntimeHost = NULL;

	HRESULT Result = CLRCreateInstance(CLSID_CLRMetaHost, IID_ICLRMetaHost, (LPVOID*)&MetaHost);

	if ( ensureMsgf(SUCCEEDED(Result), TEXT("Error creating the Swarm instance.") ))
	{
		TCHAR NetFrameworkVersion[255];
		uint32 VersionLength = 255;
		Result = MetaHost->GetVersionFromFile(SwarmInterfaceDLLPath, NetFrameworkVersion, (unsigned long*)&VersionLength);
		if (FAILED(Result))
		{
			SwarmInterfaceDLLPath = TEXT("SwarmInterface.dll");
			Result = MetaHost->GetVersionFromFile(SwarmInterfaceDLLPath, NetFrameworkVersion, (unsigned long*)&VersionLength);
		}

		if (ensureMsgf(SUCCEEDED(Result), TEXT("Invalid Swarm version.")))
		{
			ICLRRuntimeInfo *RuntimeInfo = NULL;
			Result = MetaHost->GetRuntime(NetFrameworkVersion, IID_ICLRRuntimeInfo, (LPVOID*)&RuntimeInfo);

			if (ensureMsgf(SUCCEEDED(Result), TEXT("Error requesting Swarm runtime info.")))
			{
				Result = RuntimeInfo->GetInterface(CLSID_CLRRuntimeHost, IID_ICLRRuntimeHost, (LPVOID*)&RuntimeHost);
				ensureMsgf( SUCCEEDED(Result), TEXT("Error requesting Swarm interface.") );
			}
		}
	}

	if (SUCCEEDED(Result))
	{
		Result = RuntimeHost->Start();
		ensureMsgf(SUCCEEDED(Result), TEXT("Cannot start Swarm runtime host."));
	}

	if (FAILED(Result))
	{
		ensureMsgf( false, TEXT("Error initializing Swarm interface."));
		return false;
	}

	TCHAR SwarmInterfaceDllName[MAX_PATH];
	GetModuleFileName((HINSTANCE)&__ImageBase, SwarmInterfaceDllName, MAX_PATH);

	uint32 ReturnValue = 0;
	Result = RuntimeHost->ExecuteInDefaultAppDomain(SwarmInterfaceDLLPath, TEXT("NSwarm.FSwarmInterface"), TEXT("InitCppBridgeCallbacks"), SwarmInterfaceDllName, (unsigned long*)&ReturnValue);

	if (FAILED(Result))
	{
		ensureMsgf(false, TEXT("Error creating Swarm instance bridge."));
		return false;
	}
#endif // PLATFORM_WINDOWS && !PLATFORM_CPU_ARM_FAMILY

	return true;
}

///////////////////////////////////////////////////////////////////////////////
///////////////////////////////////////////////////////////////////////////////

}	// namespace NSwarm
