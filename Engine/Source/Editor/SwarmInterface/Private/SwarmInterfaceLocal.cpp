// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/PlatformProcess.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Misc/Guid.h"
#include "Misc/App.h"
#include "Async/TaskGraphInterfaces.h"

#include "SwarmInterface.h"
#include "SwarmMessages.h"

#if USE_LOCAL_SWARM_INTERFACE
	#include "IMessageContext.h"
	#include "MessageEndpoint.h"
	#include "MessageEndpointBuilder.h"
	#include "Sockets.h"
	#include "SocketSubsystem.h"
	#include "Interfaces/IPv4/IPv4Address.h"
	#include "Interfaces/IPv4/IPv4Endpoint.h"
#endif


namespace NSwarm
{

/**
 * The C++ implementation of FSwarmInterface that's not using .NET and works only for local builds
 */
class FSwarmInterfaceLocalImpl : public FSwarmInterface
{
public:
	FSwarmInterfaceLocalImpl( void );
	virtual ~FSwarmInterfaceLocalImpl( void );
	virtual int32 OpenConnection( FConnectionCallback CallbackFunc, void* CallbackData, TLogFlags LoggingFlags, const TCHAR* OptionsFolder );
	virtual int32 CloseConnection( void );
	virtual int32 SendMessage( const FMessage& Message );
	virtual int32 AddChannel( const TCHAR* FullPath, const TCHAR* ChannelName );
	virtual int32 TestChannel( const TCHAR* ChannelName );
	virtual int32 OpenChannel( const TCHAR* ChannelName, TChannelFlags ChannelFlags );
	virtual int32 CloseChannel( int32 Channel );
	virtual int32 WriteChannel( int32 Channel, const void* Data, int32 DataSize );
	virtual int32 ReadChannel( int32 Channel, void* Data, int32 DataSize );
	virtual int32 OpenJob( const FGuid& JobGuid );
	virtual int32 BeginJobSpecification( const FJobSpecification& Specification32, const FJobSpecification& Specification64 );
	virtual int32 AddTask( const FTaskSpecification& Specification );
	virtual int32 EndJobSpecification( void );
	virtual int32 CloseJob( void );
	virtual int32 Log( TVerbosityLevel Verbosity, TLogColour TextColour, const TCHAR* Message );
	virtual void SetJobGuid( const FGuid& JobGuid );
	virtual bool IsJobProcessRunning( int32* OutStatus );

private:

	int32 PrepareJobFiles();
	bool CopyJobFile( const TCHAR* FilePath );
	void PrepareTasksList();

#if USE_LOCAL_SWARM_INTERFACE
	void HandlePingMessage( const FSwarmPingMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context );
	void HandlePongMessage( const FSwarmPongMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context );
	void HandleInfoMessage( const FSwarmInfoMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context );
	void HandleAlertMessage( const FSwarmAlertMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context );
	void HandleTimingMessage( const FSwarmTimingMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context );
	void HandleTaskRequestReleaseMessage( const FSwarmTaskRequestReleaseMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context );
	void HandleTaskRequestReservationMessage( const FSwarmTaskRequestReservationMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context );
	void HandleTaskRequestSpecificationMessage( const FSwarmTaskRequestSpecificationMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context );
	void HandleJobStateMessage( const FSwarmJobStateMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context );
	void HandleTaskStateMessage( const FSwarmTaskStateMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context );
	void HandleQuitMessage( const FSwarmQuitMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context );
#endif

	FString JobFolder;
	FJobSpecification JobSpecification;
	TArray<FArchive*> Channels;
	TArray<FTaskSpecification> Tasks;
	FCriticalSection TasksCriticalSection;
	FConnectionCallback CallbackFunc;
	void* CallbackData;
#if USE_LOCAL_SWARM_INTERFACE
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;
	FMessageAddress Recipient;
	bool bIsConnected;
	bool bIsEditor;
	FProcHandle LightmassProcHandle;

	template<typename MessageType>
	void MessageSendReliable(MessageType* Message, const FMessageAddress& RecipientAddr)
	{
		MessageEndpoint->Send(Message, MessageType::StaticStruct(), EMessageFlags::Reliable,
			nullptr, TArrayBuilder<FMessageAddress>().Add(RecipientAddr), FTimespan::Zero(), FDateTime::MaxValue());
	}
#endif
};

#if USE_LOCAL_SWARM_INTERFACE
/**
 * @return The Swarm singleton
 */
FSwarmInterface* FSwarmInterface::GInstance = NULL;

bool FSwarmInterface::Initialize(const TCHAR* SwarmInterfacePath)
{
	if( GInstance == NULL )
	{
		GInstance = new FSwarmInterfaceLocalImpl();
	}

	FIPv4Endpoint::Initialize();

	return true;
}

FSwarmInterface& FSwarmInterface::Get( void )
{
	return( *GInstance );
}
#endif

FSwarmInterfaceLocalImpl::FSwarmInterfaceLocalImpl( void )
:	CallbackFunc( NULL )
,	CallbackData( NULL )
#if USE_LOCAL_SWARM_INTERFACE
,	bIsConnected( false )
,	bIsEditor( false )
,	LightmassProcHandle()
#endif
{
}

FSwarmInterfaceLocalImpl::~FSwarmInterfaceLocalImpl( void )
{
}

#if USE_LOCAL_SWARM_INTERFACE
namespace SwarmInterfaceLocalImpl
{
	bool CanUseUMB()
	{
		bool bCanUse = false;

		ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
		if (SocketSubsystem)
		{
			// create socket
			TSharedRef<FInternetAddr> BindAddr = FIPv4Endpoint::Any.ToInternetAddr();
			FSocket* Socket = SocketSubsystem->CreateSocket(NAME_DGram, TEXT("TestSocket"), BindAddr->GetProtocolType());
			if (Socket)
			{
				if (Socket->Bind(*BindAddr))
				{
					if (Socket->SetBroadcast(true) && Socket->SetMulticastLoopback(true))
					{
						// should mirror UDP_MESSAGING_DEFAULT_MULTICAST_ENDPOINT
						if (Socket->JoinMulticastGroup(*FIPv4Endpoint(FIPv4Address(230, 0, 0, 1), 6666).ToInternetAddr()))
						{
							bCanUse = true;
						}
					}
				}

				SocketSubsystem->DestroySocket(Socket);
			}
		}

		return bCanUse;
	}
}
#endif

int32 FSwarmInterfaceLocalImpl::OpenConnection( FConnectionCallback InCallbackFunc, void* InCallbackData, TLogFlags LoggingFlags, const TCHAR* OptionsFolder )
{
	// InCallbackFunc can be NULL
	// InCallbackData can be NULL
	CallbackFunc = InCallbackFunc;
	CallbackData = InCallbackData;

#if USE_LOCAL_SWARM_INTERFACE
	bIsEditor = !FString(FPlatformProcess::ExecutableName()).StartsWith(TEXT("UnrealLightmass"));

	if (!MessageEndpoint.IsValid())
	{
		MessageEndpoint = FMessageEndpoint::Builder("FSwarmInterfaceLocal")
			.Handling<FSwarmPingMessage>(this, &FSwarmInterfaceLocalImpl::HandlePingMessage)
			.Handling<FSwarmPongMessage>(this, &FSwarmInterfaceLocalImpl::HandlePongMessage)
			.Handling<FSwarmInfoMessage>(this, &FSwarmInterfaceLocalImpl::HandleInfoMessage)
			.Handling<FSwarmAlertMessage>(this, &FSwarmInterfaceLocalImpl::HandleAlertMessage)
			.Handling<FSwarmTimingMessage>(this, &FSwarmInterfaceLocalImpl::HandleTimingMessage)
			.Handling<FSwarmTaskRequestReleaseMessage>(this, &FSwarmInterfaceLocalImpl::HandleTaskRequestReleaseMessage)
			.Handling<FSwarmTaskRequestReservationMessage>(this, &FSwarmInterfaceLocalImpl::HandleTaskRequestReservationMessage)
			.Handling<FSwarmTaskRequestSpecificationMessage>(this, &FSwarmInterfaceLocalImpl::HandleTaskRequestSpecificationMessage)
			.Handling<FSwarmJobStateMessage>(this, &FSwarmInterfaceLocalImpl::HandleJobStateMessage)
			.Handling<FSwarmTaskStateMessage>(this, &FSwarmInterfaceLocalImpl::HandleTaskStateMessage)
			.Handling<FSwarmQuitMessage>(this, &FSwarmInterfaceLocalImpl::HandleQuitMessage);
		if (MessageEndpoint.IsValid())
		{
			MessageEndpoint->Subscribe<FSwarmPingMessage>();
			MessageEndpoint->Publish(FMessageEndpoint::MakeMessage<FSwarmPingMessage>(), EMessageScope::Network);

			// UMB does not allow us to identify early its initialization errors - check that manually.
			bIsConnected = SwarmInterfaceLocalImpl::CanUseUMB();
		}
		else
		{
			UE_LOGF(LogInit, Error, "Could not open local SwarmInterface connection");
		}
	}

	PrepareTasksList();

	return bIsConnected ? 1 : -1;
#else
	return 1;
#endif
}

int32 FSwarmInterfaceLocalImpl::CloseConnection( void )
{
#if USE_LOCAL_SWARM_INTERFACE
	if( LightmassProcHandle.IsValid() )
	{
		FPlatformProcess::TerminateProc(LightmassProcHandle, true);
		FPlatformProcess::CloseProc(LightmassProcHandle);
	}
	Recipient = FMessageAddress();
	MessageEndpoint.Reset();
	bIsConnected = false;
	CallbackFunc = NULL;
	CallbackData = NULL;
#endif
	return 0;
}

int32 FSwarmInterfaceLocalImpl::SendMessage( const FMessage& Message )
{
#if USE_LOCAL_SWARM_INTERFACE
	const double kMaxTimeToWaitSec = 60;
	double TimeStartedWaiting = FPlatformTime::Seconds();

	while (bIsConnected && !Recipient.IsValid())
	{
		MessageEndpoint->Publish(FMessageEndpoint::MakeMessage<FSwarmPingMessage>(), EMessageScope::Network);
		
		FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
		FPlatformProcess::Sleep(0.5f);

		double TimeWaitingSec = FPlatformTime::Seconds() - TimeStartedWaiting;
		if (TimeWaitingSec >= kMaxTimeToWaitSec)
		{
			UE_LOGF(LogInit, Error, "Timed out waiting for the recipient (TimeWaitingSec = %f)", TimeWaitingSec);
			return -1;
		}
	}

	if (Message.Type == MESSAGE_INFO)
	{
		FInfoMessage* InfoMessage = (FInfoMessage*)&Message;
		MessageSendReliable(FMessageEndpoint::MakeMessage<FSwarmInfoMessage>(InfoMessage->TextMessage), Recipient);
	}
	else if (Message.Type == MESSAGE_ALERT)
	{
		FAlertMessage* AlertMessage = (FAlertMessage*)&Message;
		MessageSendReliable(FMessageEndpoint::MakeMessage<FSwarmAlertMessage>(AlertMessage->JobGuid, AlertMessage->AlertLevel, AlertMessage->ObjectGuid, AlertMessage->TypeId, AlertMessage->TextMessage), Recipient);
	}
	else if (Message.Type == MESSAGE_TIMING)
	{
		FTimingMessage* TimingMessage = (FTimingMessage*)&Message;
		MessageSendReliable(FMessageEndpoint::MakeMessage<FSwarmTimingMessage>(TimingMessage->State, TimingMessage->ThreadNum), Recipient);
	}
	else if (Message.Type == MESSAGE_TASK_REQUEST)
	{
		FScopeLock ScopedLock(&TasksCriticalSection);
		if (Tasks.Num() == 0)
		{
			FTaskRequestResponse Response(RESPONSE_TYPE_RELEASE);
			CallbackFunc( (FMessage*)&Response, CallbackData );
		}
		else
		{
			FTaskSpecification TaskSpec = Tasks.Pop();
			CallbackFunc( (FMessage*)&TaskSpec, CallbackData );
		}
		MessageSendReliable(FMessageEndpoint::MakeMessage<FSwarmTaskRequestMessage>(), Recipient);
	}
	else if (Message.Type == MESSAGE_TASK_REQUEST_RESPONSE)
	{
		FTaskRequestResponse* ResponseMessage = (FTaskRequestResponse*)&Message;
		if (ResponseMessage->ResponseType == RESPONSE_TYPE_RELEASE)
		{
			MessageSendReliable(FMessageEndpoint::MakeMessage<FSwarmTaskRequestReleaseMessage>(), Recipient);
		}
		else if (ResponseMessage->ResponseType == RESPONSE_TYPE_RESERVATION)
		{
			MessageSendReliable(FMessageEndpoint::MakeMessage<FSwarmTaskRequestReservationMessage>(), Recipient);
		}
		else if (ResponseMessage->ResponseType == RESPONSE_TYPE_SPECIFICATION)
		{
			FTaskSpecification* SpecificationMessage = (FTaskSpecification*)ResponseMessage;
			TArray<FString> Dependencies;
			for (uint32 Index = 0; Index < SpecificationMessage->DependencyCount; Index++)
			{
				Dependencies.Add(SpecificationMessage->Dependencies[Index]);
			}
			MessageSendReliable(FMessageEndpoint::MakeMessage<FSwarmTaskRequestSpecificationMessage>(SpecificationMessage->TaskGuid, SpecificationMessage->Parameters, SpecificationMessage->Flags, Dependencies), Recipient);
		}
	}
	else if (Message.Type == MESSAGE_JOB_STATE)
	{
		FJobState* StateMessage = (FJobState*)&Message;
		MessageSendReliable(FMessageEndpoint::MakeMessage<FSwarmJobStateMessage>(StateMessage->JobGuid, StateMessage->JobState, StateMessage->JobMessage, StateMessage->JobExitCode, StateMessage->JobRunningTime), Recipient);
	}
	else if (Message.Type == MESSAGE_TASK_STATE)
	{
		FTaskState* StateMessage = (FTaskState*)&Message;
		MessageSendReliable(FMessageEndpoint::MakeMessage<FSwarmTaskStateMessage>(StateMessage->TaskGuid, StateMessage->TaskState, StateMessage->TaskMessage, StateMessage->TaskExitCode, StateMessage->TaskRunningTime), Recipient);
	}
	else if (Message.Type == MESSAGE_QUIT)
	{
		MessageSendReliable(FMessageEndpoint::MakeMessage<FSwarmQuitMessage>(), Recipient);
	}
#endif
	return 0;
}

#if USE_LOCAL_SWARM_INTERFACE
void FSwarmInterfaceLocalImpl::HandlePingMessage( const FSwarmPingMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	MessageSendReliable(FMessageEndpoint::MakeMessage<FSwarmPongMessage>(bIsEditor, FPlatformProcess::ComputerName()), Context->GetSender());
}

void FSwarmInterfaceLocalImpl::HandlePongMessage( const FSwarmPongMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	if (!Recipient.IsValid() && Message.bIsEditor != bIsEditor && Message.ComputerName == FPlatformProcess::ComputerName())
	{
		Recipient = Context->GetSender();
	}
}

void FSwarmInterfaceLocalImpl::HandleInfoMessage( const FSwarmInfoMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	if( CallbackFunc )
	{
		FInfoMessage InfoMessage(*Message.TextMessage);
		CallbackFunc((FMessage*)&InfoMessage, CallbackData);
	}
}

void FSwarmInterfaceLocalImpl::HandleAlertMessage( const FSwarmAlertMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	if( CallbackFunc )
	{
		FAlertMessage AlertMessage(Message.JobGuid, (TAlertLevel)Message.AlertLevel, Message.ObjectGuid, Message.TypeId, *Message.TextMessage);
		CallbackFunc((FMessage*)&AlertMessage, CallbackData);
	}
}

void FSwarmInterfaceLocalImpl::HandleTimingMessage( const FSwarmTimingMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	if( CallbackFunc )
	{
		FTimingMessage TimingMessage((TProgressionState)Message.State, Message.ThreadNum);
		CallbackFunc((FMessage*)&TimingMessage, CallbackData);
	}
}

void FSwarmInterfaceLocalImpl::HandleTaskRequestReleaseMessage( const FSwarmTaskRequestReleaseMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	if( CallbackFunc )
	{
		FTaskRequestResponse TaskRequestReleaseMessage(RESPONSE_TYPE_RELEASE);
		CallbackFunc((FMessage*)&TaskRequestReleaseMessage, CallbackData);
	}
}

void FSwarmInterfaceLocalImpl::HandleTaskRequestReservationMessage( const FSwarmTaskRequestReservationMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	if( CallbackFunc )
	{
		FTaskRequestResponse TaskRequestReservationMessage(RESPONSE_TYPE_RESERVATION);
		CallbackFunc((FMessage*)&TaskRequestReservationMessage, CallbackData);
	}
}

void FSwarmInterfaceLocalImpl::HandleTaskRequestSpecificationMessage( const FSwarmTaskRequestSpecificationMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	if( CallbackFunc )
	{
		FTaskSpecification TaskRequestSpecificationMessage(Message.TaskGuid, *Message.Parameters, (TJobTaskFlags)Message.Flags);
		TCHAR** Dependencies = (TCHAR**)FMemory::Malloc(Message.Dependencies.Num() * sizeof(TCHAR*));
		for (int32 Index = 0; Index < Message.Dependencies.Num(); Index++)
		{
			Dependencies[Index] = (TCHAR*)FMemory::Malloc(Message.Dependencies[Index].Len() + 1);
			FMemory::Memcpy(Dependencies[Index], *Message.Dependencies[Index], Message.Dependencies[Index].Len() + 1);
		}

		TaskRequestSpecificationMessage.AddDependencies((const TCHAR**)Dependencies, Message.Dependencies.Num());
		CallbackFunc((FMessage*)&TaskRequestSpecificationMessage, CallbackData);

		for (int32 Index = 0; Index < Message.Dependencies.Num(); Index++)
		{
			FMemory::Free(Dependencies[Index]);
		}
		FMemory::Free(Dependencies);
	}
}

void FSwarmInterfaceLocalImpl::HandleJobStateMessage( const FSwarmJobStateMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	if( CallbackFunc )
	{
		FJobState JobStateMessage(Message.Guid, (TJobTaskState)Message.State);
		CallbackFunc((FMessage*)&JobStateMessage, CallbackData);
	}
}

void FSwarmInterfaceLocalImpl::HandleTaskStateMessage( const FSwarmTaskStateMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	if( CallbackFunc )
	{
		FTaskState TaskStateMessage(Message.Guid, (TJobTaskState)Message.State);
		CallbackFunc((FMessage*)&TaskStateMessage, CallbackData);
	}
}

void FSwarmInterfaceLocalImpl::HandleQuitMessage( const FSwarmQuitMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	if( CallbackFunc )
	{
		FMessage QuitMessage(MESSAGE_QUIT);
		CallbackFunc(&QuitMessage, CallbackData);
	}
}
#endif // USE_LOCAL_SWARM_INTERFACE

int32 FSwarmInterfaceLocalImpl::AddChannel( const TCHAR* FullPath, const TCHAR* ChannelName )
{
	if (FullPath == NULL)
	{
		return SWARM_ERROR_INVALID_ARG1;
	}

	if (ChannelName == NULL)
	{
		return SWARM_ERROR_INVALID_ARG2;
	}

	// @todo: seems unused when building locally
	check(0);
	return 0;
}

int32 FSwarmInterfaceLocalImpl::TestChannel( const TCHAR* ChannelName )
{
	if( ChannelName == NULL )
	{
		return SWARM_ERROR_INVALID_ARG1;
	}

	FString FullPath = JobFolder / ChannelName;
	return FPaths::FileExists(FullPath) ? SWARM_SUCCESS : SWARM_ERROR_FILE_FOUND_NOT;
}

int32 FSwarmInterfaceLocalImpl::OpenChannel( const TCHAR* ChannelName, TChannelFlags ChannelFlags )
{
	if( ChannelName == NULL )
	{
		return SWARM_ERROR_INVALID_ARG1;
	}

	FString FullPath = JobFolder / ChannelName;
	FArchive* ChannelFile = (ChannelFlags & SWARM_CHANNEL_ACCESS_WRITE) ? IFileManager::Get().CreateFileWriter(*FullPath, FILEWRITE_AllowRead) : IFileManager::Get().CreateFileReader(*FullPath);
	if (ChannelFile)
	{
		return Channels.Add(ChannelFile);
	}
	return SWARM_ERROR_CHANNEL_IO_FAILED;
}

int32 FSwarmInterfaceLocalImpl::CloseChannel( int32 Channel )
{
	if( Channel < 0 )
	{
		return SWARM_ERROR_INVALID_ARG1;
	}

	FArchive* ChannelFile = Channels[Channel];
	ChannelFile->Close();
	delete ChannelFile;
	Channels[Channel] = NULL;

	return SWARM_SUCCESS;
}

int32 FSwarmInterfaceLocalImpl::WriteChannel( int32 Channel, const void* Data, int32 DataSize )
{
	if( Channel < 0 )
	{
		return SWARM_ERROR_INVALID_ARG1;
	}

	if( Data == NULL )
	{
		return SWARM_ERROR_INVALID_ARG2;
	}

	if( DataSize < 0 )
	{
		return SWARM_ERROR_INVALID_ARG3;
	}

	if( Channels[Channel] == NULL)
	{
		return SWARM_ERROR_CHANNEL_NOT_FOUND;
	}

	Channels[Channel]->Serialize((void*)Data, DataSize);

	return DataSize;
}

int32 FSwarmInterfaceLocalImpl::ReadChannel( int32 Channel, void* Data, int32 DataSize )
{
	if( Channel < 0 )
	{
		return SWARM_ERROR_INVALID_ARG1;
	}

	if( Data == NULL )
	{
		return SWARM_ERROR_INVALID_ARG2;
	}

	if( DataSize < 0 )
	{
		return SWARM_ERROR_INVALID_ARG3;
	}

	if( Channels[Channel] == NULL)
	{
		return SWARM_ERROR_CHANNEL_NOT_FOUND;
	}

	Channels[Channel]->Serialize((void*)Data, DataSize);

	return DataSize;
}

int32 FSwarmInterfaceLocalImpl::OpenJob( const FGuid& JobGuid )
{
	int32 ErrorCode = SWARM_INVALID;

	JobFolder = FPaths::GameAgnosticSavedDir() / TEXT("Swarm") / TEXT("SwarmCache") / TEXT("Jobs") / FString::Printf(TEXT("Job-%08X-%08X-%08X-%08X"), JobGuid.A, JobGuid.B, JobGuid.C, JobGuid.D);
	if (IFileManager::Get().MakeDirectory(*JobFolder, true))
	{
		ErrorCode = SWARM_SUCCESS;
	}

	return ErrorCode;
}

int32 FSwarmInterfaceLocalImpl::BeginJobSpecification( const FJobSpecification& Specification32, const FJobSpecification& Specification64 )
{
	if( Specification32.ExecutableName == NULL && Specification64.ExecutableName == NULL )
	{
		return SWARM_ERROR_INVALID_ARG;
	}

	if( Specification32.Parameters == NULL && Specification64.Parameters == NULL )
	{
		return SWARM_ERROR_INVALID_ARG;
	}

	if( (Specification32.RequiredDependencyCount > 0 && Specification32.RequiredDependencies == NULL) ||
		(Specification32.OptionalDependencyCount > 0 && Specification32.OptionalDependencies == NULL) ||
		(Specification64.RequiredDependencyCount > 0 && Specification64.RequiredDependencies == NULL) ||
		(Specification64.OptionalDependencyCount > 0 && Specification64.OptionalDependencies == NULL) )
	{
		return SWARM_ERROR_INVALID_ARG;
	}

	JobSpecification = Specification32.ExecutableName ? Specification32 : Specification64;
	return PrepareJobFiles();
}

int32 FSwarmInterfaceLocalImpl::AddTask( const FTaskSpecification& Specification )
{
	if( Specification.Parameters == NULL )
	{
		return SWARM_ERROR_INVALID_ARG;
	}

	if( ( Specification.DependencyCount > 0 ) &&
		( Specification.Dependencies == NULL ) )
	{
		return SWARM_ERROR_INVALID_ARG;
	}

	FString TasksFolder = JobFolder / TEXT("Tasks");
	IFileManager::Get().MakeDirectory( *TasksFolder, true );

	FString TaskFileName = FString::Printf(TEXT("%08X-%08X-%08X-%08X"), Specification.TaskGuid.A, Specification.TaskGuid.B, Specification.TaskGuid.C, Specification.TaskGuid.D);
	FArchive* TaskFile = IFileManager::Get().CreateFileWriter( *( TasksFolder / TaskFileName ) );
	TaskFile->Serialize((void*)Specification.Parameters, FCString::Strlen(Specification.Parameters) * sizeof(TCHAR));
	TaskFile->Close();
	delete TaskFile;

	return 0;
}

int32 FSwarmInterfaceLocalImpl::EndJobSpecification( void )
{
#if USE_LOCAL_SWARM_INTERFACE
	if (!(JobSpecification.Flags & JOB_FLAG_MANUAL_START))
	{
		FString Parameters = FString(JobSpecification.Parameters) + (FApp::IsEngineInstalled() ? TEXT(" -installed") : TEXT(""));
		LightmassProcHandle = FPlatformProcess::CreateProc( JobSpecification.ExecutableName, *Parameters, true, false, false, NULL, 0, NULL, NULL);
		if (LightmassProcHandle.IsValid())
		{
			return 0;
		}
		return SWARM_ERROR_CONNECTION_DISCONNECTED;
	}
#endif
	return 0;
}

int32 FSwarmInterfaceLocalImpl::CloseJob( void )
{
	FScopeLock ScopedLock(&TasksCriticalSection);
	Channels.Empty();
	Tasks.Empty();
	return SWARM_SUCCESS;
}

bool FSwarmInterfaceLocalImpl::IsJobProcessRunning( int32* OutStatus )
{
#if USE_LOCAL_SWARM_INTERFACE
	const bool bIsRunning = FPlatformProcess::IsProcRunning(LightmassProcHandle);
	if (!bIsRunning && OutStatus)
	{
		FPlatformProcess::GetProcReturnCode(LightmassProcHandle, OutStatus);
	}
	return bIsRunning;
#else
	return false;
#endif
}

int32 FSwarmInterfaceLocalImpl::Log( TVerbosityLevel Verbosity, TLogColour TextColour, const TCHAR* Message )
{
	if( Message == NULL )
	{
		return SWARM_ERROR_NULL_POINTER;
	}

	// @todo
//	OutputDebugString(Message);
	return 0;
}

void FSwarmInterfaceLocalImpl::SetJobGuid( const FGuid& JobGuid )
{
	JobFolder = FPaths::GameAgnosticSavedDir() / TEXT("Swarm") / TEXT("SwarmCache") / TEXT("Jobs") / FString::Printf(TEXT("Job-%08X-%08X-%08X-%08X"), JobGuid.A, JobGuid.B, JobGuid.C, JobGuid.D);
}

int32 FSwarmInterfaceLocalImpl::PrepareJobFiles()
{
	// Currently we're running UnrealLightmass directly from Engine/Binaries/<platform>, so no need to copy the exe and dependencies
#if 0
	if( !CopyJobFile(JobSpecification.ExecutableName) )
	{
		return SWARM_ERROR_FILE_FOUND_NOT;
	}

	for( uint32 Index = 0; Index < JobSpecification.RequiredDependencyCount; Index++ )
	{
		if( !CopyJobFile(JobSpecification.RequiredDependencies[Index]) )
		{
			return SWARM_ERROR_FILE_FOUND_NOT;
		}
	}

	for( uint32 Index = 0; Index < JobSpecification.OptionalDependencyCount; Index++ )
	{
		if( !CopyJobFile(JobSpecification.OptionalDependencies[Index]) )
		{
			return SWARM_ERROR_FILE_FOUND_NOT;
		}
	}
#endif
	return SWARM_SUCCESS;
}

bool FSwarmInterfaceLocalImpl::CopyJobFile( const TCHAR* FilePath )
{
	FString FileName = FPaths::GetCleanFilename( FilePath );
	return IFileManager::Get().Copy( *(JobFolder / FileName), FilePath ) == COPY_OK;
}

void FSwarmInterfaceLocalImpl::PrepareTasksList()
{
	FString TasksFolder = JobFolder / TEXT("Tasks");
	TArray<FString> TaskFiles;
	IFileManager::Get().FindFiles(TaskFiles, *(TasksFolder / TEXT("*")), true, false);

	FScopeLock ScopedLock(&TasksCriticalSection);

	for (int32 FileIndex = 0; FileIndex < TaskFiles.Num(); FileIndex++)
	{
		FString& FileName = TaskFiles[FileIndex];
		FGuid TaskGuid;
		FGuid::Parse(FileName, TaskGuid);
		FTaskSpecification TaskSpec(TaskGuid, TEXT("TaskDesc"), JOB_FLAG_USE_DEFAULTS);
		Tasks.Push(TaskSpec);
	}
}

}	// namespace NSwarm
