// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "SwarmDefines.h"
#include "SwarmMessages.generated.h"


USTRUCT()
struct FSwarmPingMessage
{
	GENERATED_USTRUCT_BODY()
};


USTRUCT()
struct FSwarmPongMessage
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category="Message")
	bool bIsEditor;

	UPROPERTY(EditAnywhere, Category="Message")
	FString ComputerName;

	FSwarmPongMessage()
		: bIsEditor(false)
	{ }

	FSwarmPongMessage(bool bInIsEditor, const TCHAR* InComputerName)
		: bIsEditor(bInIsEditor)
		, ComputerName(InComputerName)
	{ }
};


USTRUCT()
struct FSwarmInfoMessage
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category="Message")
	FString TextMessage;

	FSwarmInfoMessage() { }

	FSwarmInfoMessage(const FString& InTextMessage)
		: TextMessage(InTextMessage)
	{ }
};


USTRUCT()
struct FSwarmAlertMessage
{
	GENERATED_USTRUCT_BODY()

	/** The Job Guid */
	UPROPERTY(EditAnywhere, Category="Message")
	FGuid JobGuid;

	/** The type of alert */
	UPROPERTY(EditAnywhere, Category="Message")
	uint8 AlertLevel;
	
	/** The identifier for the object that is associated with the issue */
	UPROPERTY(EditAnywhere, Category="Message")
	FGuid ObjectGuid;
	
	/** App-specific identifier for the type of the object */
	UPROPERTY(EditAnywhere, Category="Message")
	int32 TypeId;
	
	/** Generic text message for informational purposes */
	UPROPERTY(EditAnywhere, Category="Message")
	FString TextMessage;

	FSwarmAlertMessage() 
		: AlertLevel(0)
		, TypeId(0)
	{ }

	FSwarmAlertMessage(const FGuid& InJobGuid, NSwarm::TAlertLevel InAlertLevel, const FGuid& InObjectGuid, int32 InTypeId, const FString& InTextMessage)
		: JobGuid(InJobGuid)
		, AlertLevel(InAlertLevel)
		, ObjectGuid(InObjectGuid)
		, TypeId(InTypeId)
		, TextMessage(InTextMessage)
	{ }
};


USTRUCT()
struct FSwarmTimingMessage
{
	GENERATED_USTRUCT_BODY()

	/** State that the distributed job is transitioning to */
	UPROPERTY(EditAnywhere, Category="Message")
	uint8 State;

	/** The thread this state is referring to */
	UPROPERTY(EditAnywhere, Category="Message")
	int32 ThreadNum;

	FSwarmTimingMessage()
		: State(0)
		, ThreadNum(0)
	{ }

	FSwarmTimingMessage(NSwarm::TProgressionState InState, int32 InThreadNum)
		: State(InState)
		, ThreadNum(InThreadNum)
	{ }
};


USTRUCT()
struct FSwarmTaskRequestMessage
{
	GENERATED_USTRUCT_BODY()

	FSwarmTaskRequestMessage() { }
};


USTRUCT()
struct FSwarmTaskRequestReleaseMessage
{
	GENERATED_USTRUCT_BODY()

	FSwarmTaskRequestReleaseMessage() { }
};


USTRUCT()
struct FSwarmTaskRequestReservationMessage
{
	GENERATED_USTRUCT_BODY()

	FSwarmTaskRequestReservationMessage() { }
};


USTRUCT()
struct FSwarmTaskRequestSpecificationMessage
{
	GENERATED_USTRUCT_BODY()

	/** The GUID used for identifying the Task being referred to */
	UPROPERTY(EditAnywhere, Category="Message")
	FGuid TaskGuid;

	/** The Task's parameter string specified with AddTask */
	UPROPERTY(EditAnywhere, Category="Message")
	FString Parameters;

	/** Flags used to control the behavior of the Task, subject to overrides from the containing Job */
	UPROPERTY(EditAnywhere, Category="Message")
	uint8 Flags;

	/** The Task's cost, relative to all other Tasks in the same Job, used for even distribution and scheduling */
	UPROPERTY(EditAnywhere, Category="Message")
	uint32 Cost;

	/** Any additional Task dependencies */
	UPROPERTY(EditAnywhere, Category="Message")
	TArray<FString> Dependencies;

	FSwarmTaskRequestSpecificationMessage()
		: Flags(0)
		, Cost(0)
	{ }

	FSwarmTaskRequestSpecificationMessage(FGuid InTaskGuid, const FString& InParameters, NSwarm::TJobTaskFlags InFlags, const TArray<FString>& InDependencies)
		: TaskGuid(InTaskGuid)
		, Parameters(InParameters)
		, Flags(InFlags)
		, Dependencies(InDependencies)
	{ }
};


USTRUCT()
struct FSwarmJobStateMessage
{
	GENERATED_USTRUCT_BODY()

	/** The Job GUID used for identifying the Job */
	UPROPERTY(EditAnywhere, Category="Message")
	FGuid Guid;

	/** The current state and arbitrary message */
	UPROPERTY(EditAnywhere, Category="Message")
	uint8 State;

	UPROPERTY(EditAnywhere, Category="Message")
	FString Message;

	/** Various stats, including run time, exit codes, etc. */
	UPROPERTY(EditAnywhere, Category="Message")
	int32 ExitCode;

	UPROPERTY(EditAnywhere, Category="Message")
	double RunningTime;

	FSwarmJobStateMessage()
		: State(0)
		, ExitCode(0)
		, RunningTime(0.0)
	{ }

	FSwarmJobStateMessage(FGuid InGuid, NSwarm::TJobTaskState InState, const FString& InMessage, int32 InExitCode, double InRunningTime)
		: Guid(InGuid)
		, State(InState)
		, Message(InMessage)
		, ExitCode(InExitCode)
		, RunningTime(InRunningTime)
	{ }
};


USTRUCT()
struct FSwarmTaskStateMessage
{
	GENERATED_USTRUCT_BODY()

	/** The Task GUID used for identifying the Task */
	UPROPERTY(EditAnywhere, Category="Message")
	FGuid Guid;

	/** The current state and arbitrary message */
	UPROPERTY(EditAnywhere, Category="Message")
	uint8 State;

	UPROPERTY(EditAnywhere, Category="Message")
	FString Message;

	/** Various stats, including run time, exit codes, etc. */
	UPROPERTY(EditAnywhere, Category="Message")
	int32 ExitCode;

	UPROPERTY(EditAnywhere, Category="Message")
	double RunningTime;

	FSwarmTaskStateMessage()
		: State(0)
		, ExitCode()
		, RunningTime(0.0)
	{ }

	FSwarmTaskStateMessage(FGuid InGuid, NSwarm::TJobTaskState InState, const FString& InMessage, int32 InExitCode, double InRunningTime)
		: Guid(InGuid)
		, State(InState)
		, Message(InMessage)
		, ExitCode(InExitCode)
		, RunningTime(InRunningTime)
	{ }
};


USTRUCT()
struct FSwarmQuitMessage
{
	GENERATED_USTRUCT_BODY()

	FSwarmQuitMessage() { }
};
