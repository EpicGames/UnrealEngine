// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace NSwarm
{

enum
{
	SWARM_SUCCESS							= 0,

	SWARM_INVALID							= -1,
	SWARM_ERROR_FILE_FOUND_NOT				= -2,
	SWARM_ERROR_NULL_POINTER				= -3,
	SWARM_ERROR_EXCEPTION					= -4,
	SWARM_ERROR_INVALID_ARG					= -5,
	SWARM_ERROR_INVALID_ARG1				= -6,
	SWARM_ERROR_INVALID_ARG2				= -7,
	SWARM_ERROR_INVALID_ARG3				= -8,
	SWARM_ERROR_INVALID_ARG4				= -9,
	SWARM_ERROR_CHANNEL_NOT_FOUND			= -10,
	SWARM_ERROR_CHANNEL_NOT_READY			= -11,
	SWARM_ERROR_CHANNEL_IO_FAILED			= -12,
	SWARM_ERROR_CONNECTION_NOT_FOUND		= -13,
	SWARM_ERROR_JOB_NOT_FOUND				= -14,
	SWARM_ERROR_JOB							= -15,
	SWARM_ERROR_CONNECTION_DISCONNECTED		= -16,
	SWARM_ERROR_AGENT_NOT_FOUND				= -17,
};

/**
 * Consistent version enum used by messages, Jobs, Tasks, etc.
 */
enum TSwarmVersionValue
{
	VERSION_INVALID							= 0x00000000,
	VERSION_1_0								= 0x00000010,
};

/**
 * Flags to determine the level of logging
 */
enum TLogFlags
{
	SWARM_LOG_NONE							= 0,
	SWARM_LOG_TIMINGS						= ( 1 << 0 ),
	SWARM_LOG_CONNECTIONS					= ( 1 << 1 ),
	SWARM_LOG_CHANNELS						= ( 1 << 2 ),
	SWARM_LOG_MESSAGES						= ( 1 << 3 ),
	SWARM_LOG_JOBS							= ( 1 << 4 ),
	SWARM_LOG_TASKS							= ( 1 << 5 ),

	SWARM_LOG_ALL							= SWARM_LOG_TIMINGS | SWARM_LOG_CONNECTIONS | SWARM_LOG_CHANNELS | SWARM_LOG_MESSAGES | SWARM_LOG_JOBS | SWARM_LOG_TASKS,
};

/**
 * The level of debug info spewed to the log files
 */
enum TVerbosityLevel
{
	VERBOSITY_Silent						= 0,
	VERBOSITY_Critical,
	VERBOSITY_Simple,
	VERBOSITY_Informative,
	VERBOSITY_Complex,
	VERBOSITY_Verbose,
	VERBOSITY_ExtraVerbose,
	VERBOSITY_SuperVerbose,
};

// Available colours used in log text.
enum TLogColour
{
    LOGCOLOUR_Green = 0,
    LOGCOLOUR_Red,
    LOGCOLOUR_Orange,
    LOGCOLOUR_Blue,
};

/**
 * The current state of the lighting build process
 */
enum TProgressionState
{
	PROGSTATE_TaskTotal						= 0,			
	PROGSTATE_TasksInProgress,	
	PROGSTATE_TasksCompleted,		
	PROGSTATE_Idle,				
	PROGSTATE_InstigatorConnected,
	PROGSTATE_RemoteConnected,
	PROGSTATE_Exporting,
	PROGSTATE_BeginJob,
	PROGSTATE_Blocked,
	PROGSTATE_Preparing0,
	PROGSTATE_Preparing1,
	PROGSTATE_Preparing2,
	PROGSTATE_Preparing3,
	PROGSTATE_Processing0,
	PROGSTATE_Processing1,
	PROGSTATE_Processing2,
	PROGSTATE_Processing3,
	PROGSTATE_FinishedProcessing0,
	PROGSTATE_FinishedProcessing1,
	PROGSTATE_FinishedProcessing2,
	PROGSTATE_FinishedProcessing3,
	PROGSTATE_ExportingResults,
	PROGSTATE_ImportingResults,
	PROGSTATE_Finished,
	PROGSTATE_RemoteDisconnected,
	PROGSTATE_InstigatorDisconnected,
	PROGSTATE_Preparing4,
	PROGSTATE_Num
};

/**
 * Flags that define the intended behavior of the channel. The most
 * important of which are whether the channel is read or write, and
 * whether it's a general, persistent cache channel, or whether it's
 * a job-specific channel. Additional misc flags are available as
 * well.
 */
enum TChannelFlags
{
	SWARM_CHANNEL_TYPE_PERSISTENT			= 0x00000001,
	SWARM_CHANNEL_TYPE_JOB_ONLY				= 0x00000002,
	SWARM_CHANNEL_TYPE_MASK					= 0x0000000F,

	SWARM_CHANNEL_ACCESS_READ				= 0x00000010,
	SWARM_CHANNEL_ACCESS_WRITE				= 0x00000020,
	SWARM_CHANNEL_ACCESS_MASK				= 0x000000F0,

	// The combinations we care about are easiest to just special case here
	SWARM_CHANNEL_READ						= SWARM_CHANNEL_TYPE_PERSISTENT | SWARM_CHANNEL_ACCESS_READ,
	SWARM_CHANNEL_WRITE						= SWARM_CHANNEL_TYPE_PERSISTENT | SWARM_CHANNEL_ACCESS_WRITE,
	SWARM_JOB_CHANNEL_READ					= SWARM_CHANNEL_TYPE_JOB_ONLY | SWARM_CHANNEL_ACCESS_READ,
	SWARM_JOB_CHANNEL_WRITE					= SWARM_CHANNEL_TYPE_JOB_ONLY | SWARM_CHANNEL_ACCESS_WRITE,

	// Any additional flags for debugging or extended features
	SWARM_CHANNEL_MISC_ENABLE_PAPER_TRAIL	= 0x00010000,
	SWARM_CHANNEL_MISC_ENABLE_COMPRESSION	= 0x00020000,
	SWARM_CHANNEL_MISC_MASK					= 0x000F0000,
};

enum TMessageType
{
	MESSAGE_NONE							= 0x00000000,
	MESSAGE_INFO							= 0x00000001,
	MESSAGE_ALERT							= 0x00000002,
	MESSAGE_TIMING							= 0x00000003,
	MESSAGE_PING							= 0x00000004,
	MESSAGE_SIGNAL							= 0x00000005,

	/** Job messages */
	MESSAGE_JOB_SPECIFICATION				= 0x00000010,
	MESSAGE_JOB_STATE						= 0x00000020,

	/** Task messages */
	MESSAGE_TASK_REQUEST					= 0x00000100,
	MESSAGE_TASK_REQUEST_RESPONSE			= 0x00000200,
	MESSAGE_TASK_STATE						= 0x00000300,

	MESSAGE_QUIT							= 0x00008000,
};

enum TTaskRequestResponseType
{
	RESPONSE_TYPE_RELEASE					= 0x00000001,
	RESPONSE_TYPE_RESERVATION				= 0x00000002,
	RESPONSE_TYPE_SPECIFICATION				= 0x00000003,
};

/**
 * Flags used when creating a Job or Task
 */
enum TJobTaskFlags
{
	JOB_FLAG_USE_DEFAULTS					= 0x00000000,
	JOB_FLAG_ALLOW_REMOTE					= 0x00000001,
	JOB_FLAG_MANUAL_START					= 0x00000002,
	JOB_FLAG_64BIT							= 0x00000004,
	JOB_FLAG_MINIMIZED						= 0x00000008,

	JOB_TASK_FLAG_USE_DEFAULTS				= 0x00000000,
	JOB_TASK_FLAG_ALLOW_REMOTE				= 0x00000100,
};

/**
 * All possible states a Job or Task can be in
 */
enum TJobTaskState
{
	JOB_STATE_INVALID						= 0x00000001,
	JOB_STATE_IDLE							= 0x00000002,
	JOB_STATE_READY							= 0x00000003,
	JOB_STATE_RUNNING						= 0x00000004,
	JOB_STATE_COMPLETE_SUCCESS				= 0x00000005,
	JOB_STATE_COMPLETE_FAILURE				= 0x00000006,
	JOB_STATE_KILLED						= 0x00000007,

	JOB_TASK_STATE_INVALID					= 0x00000011,
	JOB_TASK_STATE_IDLE						= 0x00000012,
	JOB_TASK_STATE_ACCEPTED					= 0x00000013,
	JOB_TASK_STATE_REJECTED					= 0x00000014,
	JOB_TASK_STATE_RUNNING					= 0x00000015,
	JOB_TASK_STATE_COMPLETE_SUCCESS			= 0x00000016,
	JOB_TASK_STATE_COMPLETE_FAILURE			= 0x00000017,
	JOB_TASK_STATE_KILLED					= 0x00000018,
};

/**
 *	The alert levels
 */
enum TAlertLevel
{
	ALERT_LEVEL_INFO						= 0x00000001,
	ALERT_LEVEL_WARNING						= 0x00000002,
	ALERT_LEVEL_ERROR						= 0x00000003,
	ALERT_LEVEL_CRITICAL_ERROR	UE_DEPRECATED(5.1, "CriticalError was removed because it can't trigger an assert at the callsite. Use 'checkf' instead.") = 0x00000004,
};

}
