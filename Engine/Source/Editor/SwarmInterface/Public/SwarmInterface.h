// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/PlatformMath.h"
#include "Misc/Guid.h"

#if PLATFORM_MAC || PLATFORM_LINUX
#define USE_LOCAL_SWARM_INTERFACE 1
#else
#define USE_LOCAL_SWARM_INTERFACE 0
#endif

#include "SwarmDefines.h"

namespace NSwarm
{

/**
 * A simple base class for messages. For each version of the messaging interface
 * a newly derived type will inherit from this class. The base class is used to
 * simply carry lightweight loads for messages, i.e. just the message type, which
 * may be enough information in itself. For additional message data, subclass and
 * add any additional data there.
 */
class FMessage
{
public:
	/**
	 * Default constructor, initializes to default values
	 */
	FMessage( void )
		: Version( VERSION_1_0 )
		, Type( MESSAGE_NONE )
	{
	}

	/**
	 * Constructor, initializes to specified values
	 *
	 * @param NewType The type of the message, one of EMessageType
	 */
	FMessage( TMessageType NewType )
		: Version( VERSION_1_0 )
	,	Type( NewType )
	{
	}

	/**
	 * Constructor, initializes to specified values
	 *
	 * @param NewVersion The version of the message format; one of ESwarmVersionValue
	 * @param NewType The type of the message, one of EMessageType
	 */
	FMessage( TSwarmVersionValue NewVersion, TMessageType NewType )
		: Version( NewVersion )
		, Type( NewType )
	{
	}

	/** The version of the message format; one of ESwarmVersionValue */
	TSwarmVersionValue		Version;
	/** The type of the message, one of EMessageType */
	TMessageType			Type;
};

///////////////////////////////////////////////////////////////////////////////

/**
 * Implementation of a generic info message, which just includes generic text.
 */
class FInfoMessage : public FMessage
{
public:
	/**
	 * Constructor, initializes to default and specified values
	 */
	FInfoMessage( const TCHAR* InTextMessage )
		: FMessage( VERSION_1_0, MESSAGE_INFO )
		, TextMessage( InTextMessage )
	{
	}

	/** Generic text message for informational purposes */
	const TCHAR*	TextMessage;
};

///////////////////////////////////////////////////////////////////////////////

/**
 * Implementation of a alert message, which includes:
 *
 *	- The alert type:
 *		a) warning
 *		b) error
 *		c) critical error
 *	- The Job GUID
 *	- The GUID of the item causing the issue
 *	- A 32-bit field intended to identify the type of the item
 *	- A string giving the issue message (which will be localized on the Unreal side of things).
 */
class FAlertMessage : public FMessage
{
public:
	/**
	 * Constructor, initializes to default and specified values
	 */
	FAlertMessage( const FGuid& InJobGuid, TAlertLevel InAlertLevel, const FGuid& InObjectGuid, int InTypeId )
		: FMessage( VERSION_1_0, MESSAGE_ALERT )
		, JobGuid(InJobGuid)
		, AlertLevel(InAlertLevel)
		, ObjectGuid(InObjectGuid)
		, TypeId(InTypeId)
		, TextMessage(NULL)
	{
	}
	/**
	 * Constructor, initializes to default and specified values
	 */
	FAlertMessage( const FGuid& InJobGuid, TAlertLevel InAlertLevel, const FGuid& InObjectGuid, int InTypeId, const TCHAR* InTextMessage )
		: FMessage( VERSION_1_0, MESSAGE_ALERT )
		, JobGuid(InJobGuid)
		, AlertLevel(InAlertLevel)
		, ObjectGuid(InObjectGuid)
		, TypeId(InTypeId)
		, TextMessage( InTextMessage )
	{
	}

	/** The Job Guid */
	FGuid			JobGuid;
	/** The type of alert */
	TAlertLevel		AlertLevel;
	/** The identifier for the object that is associated with the issue */
	FGuid			ObjectGuid;
	/** App-specific identifier for the type of the object */
	int				TypeId;
	/** Generic text message for informational purposes */
	const TCHAR*	TextMessage;
};

///////////////////////////////////////////////////////////////////////////////

/**
 * Implementation of a generic info message, which just includes generic text.
 */
class FTimingMessage : public FMessage
{
public:
	/**
	 * Constructor, initializes to default and specified values
	 */
	FTimingMessage( TProgressionState NewState, int InThreadNum )
		: FMessage( VERSION_1_0, MESSAGE_TIMING )
		, State( NewState )
		, ThreadNum( InThreadNum )
	{
	}

	/** State that the distributed job is transitioning to */
	TProgressionState	State;
	/** The thread this state is referring to */
	int					ThreadNum;
};

///////////////////////////////////////////////////////////////////////////////

/**
 * Implementation of a task request response message. All uses include the GUID
 * of the Job the request referred to. Currently used for these message types:
 * 
 * TASK_RELEASE: Signifies that the requester is no longer required to process
 * any more Tasks. The requester is free to consider this Job completed.
 * 
 * TASK_RESERVATION: Sent back only if the Job specified is still active but
 * no additional Tasks are available at this time.
 *
 * TASK_SPECIFICATION: Details a Task that can be worked on
 */
class FTaskRequestResponse : public FMessage
{
public:
	/**
	 * Constructor, initializes to default and specified values
	 */
	FTaskRequestResponse( TTaskRequestResponseType NewResponseType )
		: FMessage( VERSION_1_0, MESSAGE_TASK_REQUEST_RESPONSE )
		, ResponseType( NewResponseType )
	{
	}

	/** The type of response this message is. Subclasses add any additional data */
	TTaskRequestResponseType	ResponseType;
};

///////////////////////////////////////////////////////////////////////////////

/**
 * Encapsulates information about a Task specification passed into AddTask and
 * later sent in response to a TASK_REQUEST message
 */
class FTaskSpecification : public FTaskRequestResponse
{
public:
	/**
	 * Constructor, initializes to default and specified values
	 */
	FTaskSpecification( FGuid TaskTaskGuid, const TCHAR* TaskParameters, TJobTaskFlags TaskFlags )
		: FTaskRequestResponse( RESPONSE_TYPE_SPECIFICATION )
		, TaskGuid( TaskTaskGuid )
		, Parameters( TaskParameters )
		, Flags( TaskFlags )
		, Dependencies( NULL )
		, DependencyCount( 0 )
	{
	}

	/**
	 * Used to add channel dependencies to a Task. When an Agent runs this Task,
	 * it will ensure that all dependencies are satisfied prior to giving the
	 * Task to the requester.
	 *
	 * @param NewDependencies The list of additional dependent channel names
	 * @param NewDependencyCount The number of elements in the NewDependencies list
	 */
	void AddDependencies( const TCHAR** NewDependencies, uint32 NewDependencyCount )
	{
		Dependencies = NewDependencies;
		DependencyCount = NewDependencyCount;
	}

	/** The GUID used for identifying the Task being referred to */
	FGuid				TaskGuid;

	/** The Task's parameter string specified with AddTask */
	const TCHAR*		Parameters;

	/** Flags used to control the behavior of the Task, subject to overrides from the containing Job */
	TJobTaskFlags		Flags;

	/** The Task's cost, relative to all other Tasks in the same Job, used for even distribution and scheduling */
	uint32				Cost;

	/** Any additional Task dependencies */
	const TCHAR**		Dependencies;
	uint32				DependencyCount;
};

///////////////////////////////////////////////////////////////////////////////

/**
 * Encapsulates information about a Job's state, used to communicate
 * back to the Instigator
 */
class FJobState : public FMessage
{
public:
	/**
	 * Constructor, initializes to specified values
	 */
	FJobState( FGuid NewJobGuid, TJobTaskState NewJobState )
		: FMessage( VERSION_1_0, MESSAGE_JOB_STATE )
		, JobGuid( NewJobGuid )
		, JobState( NewJobState )
		, JobMessage( NULL )
		, JobExitCode( 0 )
		, JobRunningTime( 0.0 )
	{
	}

	/** The Job GUID used for identifying the Job */
	FGuid			JobGuid;

	/** The current state and arbitrary message */
	TJobTaskState	JobState;
	const TCHAR*	JobMessage;

	/** Various stats, including run time, exit codes, etc. */
	int32			JobExitCode;
	double			JobRunningTime;
};

///////////////////////////////////////////////////////////////////////////////

/**
 * Encapsulates information about a Task's state, used to communicate
 * back to the Instigator
 */
class FTaskState : public FMessage
{
public:
	/**
	 * Constructor, initializes to specified values
	 */
	FTaskState( FGuid NewTaskGuid, TJobTaskState NewTaskState )
		: FMessage( VERSION_1_0, MESSAGE_TASK_STATE )
		, TaskGuid( NewTaskGuid )
		, TaskState( NewTaskState )
		, TaskMessage( NULL )
		, TaskExitCode( 0 )
		, TaskRunningTime( 0.0 )
	{
	}

	/** The Task GUID used for identifying the Task */
	FGuid			TaskGuid;

	/** The current Task state and arbitrary message */
	TJobTaskState	TaskState;
	const TCHAR*	TaskMessage;

	/** Various stats, including run time, exit codes, etc. */
	int32			TaskExitCode;
	double			TaskRunningTime;
};

///////////////////////////////////////////////////////////////////////////////

/**
 * Encapsulates information about a Job specification passed into BeginJobSpecification
 */
class FJobSpecification
{
public:
	/**
	 * Default constructor, initializes to an empty (invalid) job.
	 */
	FJobSpecification()
		: ExecutableName( NULL )
		, Parameters( NULL )
		, Flags( JOB_FLAG_USE_DEFAULTS )
		, RequiredDependencies( NULL )
		, RequiredDependencyCount( 0 )
		, OptionalDependencies( NULL )
		, OptionalDependencyCount( 0 )
		, DescriptionKeys( NULL )
		, DescriptionValues( NULL )
		, DescriptionCount( 0 )
	{
	}

	/**
	 * Constructor, initializes to default and specified values
	 */
	FJobSpecification( const TCHAR* JobExecutableName, const TCHAR* JobParameters, TJobTaskFlags JobFlags )
		: ExecutableName( JobExecutableName )
		, Parameters( JobParameters )
		, Flags( JobFlags )
		, RequiredDependencies( NULL )
		, RequiredDependencyCount( 0 )
		, OptionalDependencies( NULL )
		, OptionalDependencyCount( 0 )
		, DescriptionKeys( NULL )
		, DescriptionValues( NULL )
		, DescriptionCount( 0 )
	{
	}

	/**
	 * Used to add channel dependencies to a Job. When an Agent runs this Job,
	 * it will ensure that all dependencies are satisfied prior to launching
	 * the executable. Note that the Job executable is an implied dependency.
	 *
	 * @param NewRequiredDependencies The list of additional required dependent channel names
	 * @param NewRequiredDependencyCount The number of elements in the NewRequiredDependencies list
	 * @param NewOptionalDependencies The list of additional optional dependent channel names
	 * @param NewOptionalDependencyCount The number of elements in the NewOptionalDependencies list
	 */
	void AddDependencies( const TCHAR** NewRequiredDependencies, uint32 NewRequiredDependencyCount, const TCHAR** NewOptionalDependencies, uint32 NewOptionalDependencyCount )
	{
		RequiredDependencies = NewRequiredDependencies;
		RequiredDependencyCount = NewRequiredDependencyCount;
		OptionalDependencies = NewOptionalDependencies;
		OptionalDependencyCount = NewOptionalDependencyCount;
	}

	void AddDescription( const TCHAR** NewDescriptionKeys, const TCHAR** NewDescriptionValues, uint32 NewDescriptionCount )
	{
		DescriptionKeys = NewDescriptionKeys;
		DescriptionValues = NewDescriptionValues;
		DescriptionCount = NewDescriptionCount;
	}

	/** The Job's executable name and parameter string */
	const TCHAR*		ExecutableName;
	const TCHAR*		Parameters;

	/** Flags used to control the behavior of the executing Job */
	TJobTaskFlags		Flags;

	/** Any additional Job dependencies */
	const TCHAR**		RequiredDependencies;
	uint32				RequiredDependencyCount;
	const TCHAR**		OptionalDependencies;
	uint32				OptionalDependencyCount;

	/** Optional Job description values in key/value form */
	const TCHAR**		DescriptionKeys;
	const TCHAR**		DescriptionValues;
	uint32				DescriptionCount;
};

///////////////////////////////////////////////////////////////////////////////

/**
 * A simple callback used by the Agent to send messages back to the Instigator
 */
typedef void ( *FConnectionCallback )( FMessage* CallbackMessage, void* CallbackData );

///////////////////////////////////////////////////////////////////////////////

/**
 * The primary interface to the Swarm system
 */
class FSwarmInterface
{
public:
	/**
	 * @return The Swarm singleton
	 */
	SWARMINTERFACE_API static FSwarmInterface& Get( void );

	/** Initializes the Swarm singleton, must be called before FSwarmInterface::Get().
	*
	* @return A boolean indicating if the initialization was successful 
	*/
	SWARMINTERFACE_API static bool Initialize(const TCHAR* SwarmInterfacePath);

	/**
	 * Opens a new connection to the Swarm
	 *
	 * @param CallbackFunc The callback function Swarm will use to communicate back to the Instigator
	 *
	 * @return An int32 containing the error code (if < 0) or the handle (>= 0) which is useful for debugging only
	 */
	virtual int32 OpenConnection( FConnectionCallback CallbackFunc, void* CallbackData, TLogFlags LoggingFlags, const TCHAR* OptionsFolder ) = 0;

	/**
	 * Closes an existing connection to the Swarm
	 *
	 * @return int32 Error code (< 0 is an error)
	 */
	virtual int32 CloseConnection( void ) = 0;

	/**
	 * Sends a message to an Agent (return messages are sent via the FConnectionCallback)
	 *
	 * @param Message The message being sent
	 *
	 * @return int32 Error code (< 0 is an error)
	 */
	virtual int32 SendMessage( const FMessage& Message ) = 0;

	/**
	 * Adds an existing file to the cache. Note, any existing channel with the same
	 * name will be overwritten.
	 *
	 * @param FullPath The full path name to the file that should be copied into the cache
	 * @param ChannelName The name of the channel once it's in the cache
	 *
	 * @return int32 Error code (< 0 is an error)
	 */
	virtual int32 AddChannel( const TCHAR* FullPath, const TCHAR* ChannelName ) = 0;

	/**
	 * Determines if the named channel is in the cache
	 *
	 * @param ChannelName The name of the channel to look for
	 *
	 * @return int32 Error code (< 0 is an error)
	 */
	virtual int32 TestChannel( const TCHAR* ChannelName ) = 0;

	/**
	 * Opens a data channel for streaming data into the cache associated with an Agent
	 *
	 * @param ChannelName The name of the channel being opened
	 * @param ChannelFlags The mode, access, and other attributes of the channel being opened
	 *
	 * @return A handle to the opened channel (< 0 is an error), be sure to close it with CloseChannel
	 */
	virtual int32 OpenChannel( const TCHAR* ChannelName, TChannelFlags ChannelFlags ) = 0;

	/**
	 * Closes an open channel
	 *
	 * @param Channel An open channel handle, returned by OpenChannel
	 *
	 * @return int32 Error code (< 0 is an error)
	 */
	virtual int32 CloseChannel( int32 Channel ) = 0;

	/**
	 * Writes the provided data to the open channel opened for WRITE
	 *
	 * @param Channel An open channel handle, returned by OpenChannel
	 * @param Data Source buffer for the write
	 * @param Data Size of the source buffer
	 *
	 * @return The number of bytes written (< 0 is an error)
	 */
	virtual int32 WriteChannel( int32 Channel, const void* Data, int32 DataSize ) = 0;

	/**
	 * Reads data from a channel opened for READ into the provided buffer
	 *
	 * @param Channel An open channel handle, returned by OpenChannel
	 * @param Data Destination buffer for the read
	 * @param Data Size of the destination buffer
	 *
	 * @return The number of bytes read (< 0 is an error)
	 */
	virtual int32 ReadChannel( int32 Channel, void* Data, int32 DataSize ) = 0;

	/**
	 * Opens a Job session, which allows a Job to be specified, Tasks added, Job
	 * channels opened and used, etc. When the Job is complete and no more Job
	 * related data is needed from the Swarm, call CloseJob.
	 *
	 * @param JobGuid A GUID that uniquely identifies this Job, generated by the caller
	 *
	 * @return int32 Error code (< 0 is an error)
	 */
	virtual int32 OpenJob( const FGuid& JobGuid ) = 0;

	/**
	 * Begins a Job specification, which allows a series of Tasks to be specified
	 * via AddTask. When Tasks are done being specified, call EndJobSpecification.
	 *
	 * The default behavior will be to execute the Job executable with the
	 * specified parameters. If Tasks are added for the Job, they are expected
	 * to be requested by the executable run for the Job. If no Tasks are added
	 * for the Job, it is expected that the Job executable will perform its
	 * operations without additional Task input from Swarm.
	 *
	 * @param Specification32 A structure describing a new 32-bit Job (can be an empty specification)
	 * @param Specification64 A structure describing a new 64-bit Job (can be an empty specification)
	 *
	 * @return int32 Error code (< 0 is an error)
	 */
	virtual int32 BeginJobSpecification( const FJobSpecification& Specification32, const FJobSpecification& Specification64 ) = 0;

	/**
	 * Adds a Task to the current Job
	 *
	 * @param Specification A structure describing the new Task
	 *
	 * @return int32 Error code (< 0 is an error)
	 */
	virtual int32 AddTask( const FTaskSpecification& Specification ) = 0;

	/**
	 * Ends the Job specification, after which no additional Tasks may be defined. Also,
	 * this is generally the point when the Agent will validate and launch the Job executable,
	 * potentially distributing the Job to other Agents.
	 *
	 * @return int32 Error code (< 0 is an error)
	 */
	virtual int32 EndJobSpecification( void ) = 0;

	/**
	 * Ends the Job, after which all Job-related API usage (except OpenJob) will be rejected
	 *
	 * @return int32 Error code (< 0 is an error)
	 */
	virtual int32 CloseJob( void ) = 0;

	/**
	 * Adds a line of text to the Agent log window
	 *
	 * @param Verbosity the importance of this message
	 * @param TextColour the colour of the text
	 * @param Message the line of text to add
	 */
	virtual int32 Log( TVerbosityLevel Verbosity, TLogColour TextColour, const TCHAR* Message ) = 0;

	virtual void SetJobGuid( const FGuid& JobGuid )
	{
	}

	/**
	 * Checks if the job's process is still running
	 *
	 * @param OutStatus Exit status if process is no longer running
	 *
	 * @return bool true if process is still running
	 */
	virtual bool IsJobProcessRunning( int32* OutStatus )
	{
		return true;
	}

protected:
	FSwarmInterface( void )
	{
	}

	virtual ~FSwarmInterface( void ) = default;

private:
	static FSwarmInterface* GInstance;
};

///////////////////////////////////////////////////////////////////////////////
///////////////////////////////////////////////////////////////////////////////

} // namespace NSwarm
