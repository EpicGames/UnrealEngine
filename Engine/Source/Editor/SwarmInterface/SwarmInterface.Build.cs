// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class SwarmInterface : ModuleRules
{
	public SwarmInterface(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
			}
		);

		bool bHasSwarm = false;
		if (Target.Platform == UnrealTargetPlatform.Mac || Target.Platform == UnrealTargetPlatform.Linux)
		{
			PrivateIncludePathModuleNames.Add("MessagingCommon");
			// the modules below are only needed for the UMB usability check
			PublicDependencyModuleNames.Add("Sockets");
			PublicDependencyModuleNames.Add("Networking");
			bHasSwarm = true;
		}
		else if (Target.Platform == UnrealTargetPlatform.Win64 && Target.Architecture != UnrealArch.Arm64)
		{
			string NetFxSdkDir = Target.WindowsPlatform.NetFxSdkDir;
			if (NetFxSdkDir == null)
			{
				throw new BuildException("Could not find NetFxSDK install dir; this will prevent SwarmInterface from installing.  Install a version of .NET Framework SDK at 4.6.0 or higher.");
			}

			ExtraRootPath = ("NetFxSdk", NetFxSdkDir);

			string ArchFolder = Target.WindowsPlatform.Architecture.WindowsLibDir;

			PrivateIncludePaths.Add(Path.Combine(NetFxSdkDir, "include", "um"));
			PublicSystemLibraryPaths.Add(Path.Combine(NetFxSdkDir, "lib", "um", ArchFolder));
			bHasSwarm = true;
		}

		if (bHasSwarm)
		{
			// Copy the AgentInterface DLL to the same output directory as the editor DLL.
			RuntimeDependencies.Add("$(BinaryOutputDir)/AgentInterface.dll", "$(EngineDir)/Binaries/DotNET/AgentInterface.dll", StagedFileType.NonUFS);

			// Also copy the PDB, if it exists
			if(File.Exists(Path.Combine(EngineDirectory, "Binaries", "DotNET", "AgentInterface.pdb")))
			{
				RuntimeDependencies.Add("$(BinaryOutputDir)/AgentInterface.pdb", "$(EngineDir)/Binaries/DotNET/AgentInterface.pdb", StagedFileType.DebugNonUFS);
			}
		}
	}
}
